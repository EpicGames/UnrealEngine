{
  "FileVersion": 3,
  "Version": 1,
  "VersionName": "1.0",
  "FriendlyName": "MP4 (ISO/IEC 14496-12) utilities",
  "Description": "Provides helpers to work with mp4 files",
  "Category": "Media",
  "CreatedBy": "Epic Games, Inc.",
  "CreatedByURL": "https://epicgames.com",
  "EnabledByDefault": false,
  "CanContainContent": false,
  "Installed": false,
  "Modules": [
    {
      "Name": "MP4Utilities",
      "Type": "Runtime",
      "LoadingPhase": "PostEngineInit"
    },
    {
      "Name": "MP4Boxes",
      "Type": "Runtime",
      "LoadingPhase": "PostEngineInit"
    },
    {
      "Name": "MP4Muxer",
      "Type": "Runtime",
      "LoadingPhase": "PostEngineInit"
    }
  ]
}