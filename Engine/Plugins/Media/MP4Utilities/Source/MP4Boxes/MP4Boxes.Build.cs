// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class MP4Boxes : ModuleRules
	{
		public MP4Boxes(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(new string[] {
				"Core",
                "MP4Utilities"
            });
		}
	}
}
