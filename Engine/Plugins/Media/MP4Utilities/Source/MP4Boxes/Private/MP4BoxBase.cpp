// Copyright Epic Games, Inc. All Rights Reserved.

#include "MP4BoxBase.h"
#include "MP4Boxes.h"

namespace MP4Boxes
{

	TArray<uint8> FMP4BoxBase::GetBoxDataRAW() const
	{
		return BoxInfo.GetBoxDataRAW();
	}

	void FMP4BoxBase::ProcessBoxChildrenRecursively(MP4Utilities::FMP4AtomReader& InOutReader, const MP4Utilities::FMP4BoxInfo& InCurrentBoxInfo)
	{
		MP4Utilities::FMP4BoxInfo bi;
		while(InOutReader.ParseIntoBoxInfo(bi, InCurrentBoxInfo.Offset + InCurrentBoxInfo.DataOffset + InOutReader.GetCurrentOffset()))
		{
			TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Child = FMP4BoxFactory::Get().Create(AsWeak(), bi);
			if (AddChildBox(Child) && !Child->IsLeafBox() && !Child->IsListOfEntries())
			{
				MP4Utilities::FMP4AtomReader ar(Child->BoxInfo.Data);
				Child->ProcessBoxChildrenRecursively(ar, Child->BoxInfo);
			}
			InOutReader.SkipBytes(bi.Size - bi.DataOffset);
		}
	}


	bool FMP4BoxTreeParser::ParseBoxTreeInternal(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBox)
	{
		BoxTree = FMP4BoxFactory::Get().Create(InParent, InBox);
		// Parse the enclosed box recursively unless it contains
		// a list of entries that only the box knows how to parse.
		if (BoxTree.IsValid() && !BoxTree->IsLeafBox() && !BoxTree->IsListOfEntries())
		{
			// The data of this container box represents one or several other boxes.
			// We need to parse them one by one until there is no more data here.
			const TConstArrayView<uint8>& bd(BoxTree->GetBoxData());
			const uint8* BoxData = bd.GetData();
			int64 BoxBytesRemaining = bd.Num();
			int64 NextBoxOffset = BoxTree->GetBoxFileOffset() + BoxTree->GetBoxDataOffset();
			while(BoxBytesRemaining > 0)
			{
				MP4Utilities::FMP4AtomReader bh(MakeConstArrayView(BoxData, BoxBytesRemaining));
				MP4Utilities::FMP4BoxInfo bi;
				if (!bh.ParseIntoBoxInfo(bi, NextBoxOffset))
				{
					return false;
				}
				FMP4BoxTreeParser bp;
				if (!bp.ParseBoxTreeInternal(BoxTree, bi))
				{
					return false;
				}
				BoxTree->AddChildBox(MoveTemp(bp.BoxTree));
				BoxData += bi.Size;
				BoxBytesRemaining -= bi.Size;
				NextBoxOffset = bi.Offset + bi.Size;
			}
		}
		return true;
	}

	bool FMP4BoxTreeParser::ParseBoxTree(const TSharedPtr<MP4Utilities::FMP4BoxInfo, ESPMode::ThreadSafe>& InRootBox)
	{
		check(InRootBox.IsValid());
		bool bOk = ParseBoxTreeInternal(nullptr, *InRootBox);
		if (bOk && BoxTree.IsValid())
		{
			BoxTree->SetRootBoxData(InRootBox);
		}
		return bOk;
	}

} // namespace MP4Boxes
