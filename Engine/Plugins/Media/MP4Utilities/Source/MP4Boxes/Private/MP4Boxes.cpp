// Copyright Epic Games, Inc. All Rights Reserved.

#include "MP4Boxes.h"
#include "MP4Utilities.h"
#include "MP4BoxesModule.h"

namespace MP4Boxes
{

/****************************************************************************************************************************************************/
	struct FMP4BoxFTYP::FParsed
	{
		TArray<uint32> Brands;
		uint32 MinorVersion = 0;
	};
	void FMP4BoxFTYP::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		uint32 Major, Minor;
		ar.Read(Major);
		ar.Read(Minor);
		int32 NumCompatibleBrands = ar.GetNumBytesRemaining() / sizeof(uint32);
		Parsed->Brands.SetNumUninitialized(1 + NumCompatibleBrands);
		Parsed->MinorVersion = Minor;
		Parsed->Brands[0] = Major;
		for(int32 i=0; i<NumCompatibleBrands; ++i)
		{
			uint32 Compatible;
			ar.Read(Compatible);
			Parsed->Brands[1 + i] = Compatible;
		}
	}
	uint32 FMP4BoxFTYP::GetMajorBrand()
	{
		ParseIfRequired();
		return Parsed->Brands[0];
	}
	uint32 FMP4BoxFTYP::GetMinorVersion()
	{
		ParseIfRequired();
		return Parsed->MinorVersion;
	}
	TConstArrayView<uint32> FMP4BoxFTYP::GetCompatibleBrands()
	{
		ParseIfRequired();
		return Parsed->Brands.Num() > 1 ? MakeConstArrayView<uint32>(Parsed->Brands.GetData() + 1, Parsed->Brands.Num()-1) : TConstArrayView<uint32>();
	}
	TConstArrayView<uint32> FMP4BoxFTYP::GetAllBrands()
	{
		ParseIfRequired();
		return Parsed->Brands;
	}

/****************************************************************************************************************************************************/
	struct FMP4BoxMVHD::FParsed
	{
		uint64 Duration = 0;
		uint32 Flags = 0;
		uint32 Timescale = 0;
		uint8 Version = 0;
	};
	void FMP4BoxMVHD::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		ar.ReadVersionAndFlags(Parsed->Version, Parsed->Flags);
		if (Parsed->Version == 1)
		{
			ar.SkipBytes(16);				// `creation_time` and `modification_time`
			ar.Read(Parsed->Timescale);
			ar.Read(Parsed->Duration);
		}
		else
		{
			ar.SkipBytes(8);				// `creation_time` and `modification_time`
			ar.Read(Parsed->Timescale);
			uint32 Dur;
			ar.Read(Dur);
			Parsed->Duration = Dur == TNumericLimits<uint32>::Max() ? TNumericLimits<uint64>::Max() : Dur;
		}
	}
	MP4Utilities::FFractionalTime FMP4BoxMVHD::GetDuration()
	{
		ParseIfRequired();
		// If not specified (all 1's) or the numerator won't fit into an int64 of the MP4Utilities::FFractionalTime
		// we return an invalid time.
		if (Parsed->Duration == TNumericLimits<uint64>::Max() || Parsed->Duration > 0x7fffffffffffffffULL)
		{
			return MP4Utilities::FFractionalTime();
		}
		return MP4Utilities::FFractionalTime((int64) Parsed->Duration, Parsed->Timescale);
	}
	uint32 FMP4BoxMVHD::GetTimescale()
	{
		ParseIfRequired();
		return Parsed->Timescale;
	}

/****************************************************************************************************************************************************/
	struct FMP4BoxTKHD::FParsed
	{
		TArray<int32> Matrix;
		int64 Duration = 0;
		uint32 Flags = 0;
		uint32 TrackID = 0;
		uint32 Width = 0;
		uint32 Height = 0;
		int16 Layer = 0;
		int16 AlternateGroup = 0;
		int16 Volume = 0;
		uint8 Version = 0;
	};
	void FMP4BoxTKHD::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		ar.ReadVersionAndFlags(Parsed->Version, Parsed->Flags);
		if (Parsed->Version == 1)
		{
			ar.SkipBytes(16);				// `creation_time` and `modification_time`
			ar.Read(Parsed->TrackID);
			ar.SkipBytes(4);				// `reserved`
			uint64 Dur;
			ar.Read(Dur);
			Parsed->Duration = Dur > (uint64)TNumericLimits<int64>::Max() ? TNumericLimits<int64>::Max() : (int64)Dur;
		}
		else
		{
			ar.SkipBytes(8);				// `creation_time` and `modification_time`
			ar.Read(Parsed->TrackID);
			ar.SkipBytes(4);				// `reserved`
			uint32 Dur;
			ar.Read(Dur);
			Parsed->Duration = Dur == TNumericLimits<uint32>::Max() ? TNumericLimits<int64>::Max() : (int64)Dur;
		}
		ar.SkipBytes(8);					// `reserved` (uint32 * 2)
		uint16 Value16;
		ar.Read(Value16);
		Parsed->Layer = (int16) Value16;
		ar.Read(Value16);
		Parsed->AlternateGroup = (int16) Value16;
		ar.Read(Value16);
		Parsed->Volume = (int16) Value16;
		ar.SkipBytes(2);					// `reserved`
		Parsed->Matrix.SetNumUninitialized(9);
		for(int32 i=0; i<9; ++i)
		{
			uint32 Value32;
			ar.Read(Value32);
			Parsed->Matrix[i] = (int32) Value32;
		}
		ar.Read(Parsed->Width);
		ar.Read(Parsed->Height);
	}

	uint32 FMP4BoxTKHD::GetFlags()
	{
		ParseIfRequired();
		return Parsed->Flags;
	}

	int64 FMP4BoxTKHD::GetDuration()
	{
		ParseIfRequired();
		return Parsed->Duration;
	}

	uint32 FMP4BoxTKHD::GetTrackID()
	{
		ParseIfRequired();
		return Parsed->TrackID;
	}

	uint16 FMP4BoxTKHD::GetWidth()
	{
		ParseIfRequired();
		return Parsed->Width >> 16;
	}

	uint16 FMP4BoxTKHD::GetHeight()
	{
		ParseIfRequired();
		return Parsed->Height >> 16;
	}

	TArray<int32> FMP4BoxTKHD::GetMatrix()
	{
		ParseIfRequired();
		return Parsed->Matrix;
	}
	uint32 FMP4BoxTKHD::GetWidthRaw()
	{
		ParseIfRequired();
		return Parsed->Width;
	}
	uint32 FMP4BoxTKHD::GetHeightRaw()
	{
		ParseIfRequired();
		return Parsed->Height;
	}
	uint16 FMP4BoxTKHD::GetVolumeRaw()
	{
		ParseIfRequired();
		return Parsed->Volume;
	}


/****************************************************************************************************************************************************/

	struct FMP4BoxTREF::FParsed
	{
		TArray<FEntry> Entries;
		uint32 Flags = 0;
		uint8 Version = 0;
	};
	void FMP4BoxTREF::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		MP4Utilities::FMP4BoxInfo bi;
		while(ar.ParseIntoBoxInfo(bi, BoxInfo.Offset + BoxInfo.DataOffset + ar.GetCurrentOffset()))
		{
			FEntry& e = Parsed->Entries.Emplace_GetRef();
			e.Type = bi.Type;
			int32 NumReferences = (bi.Size - bi.DataOffset) / 4;
			e.TrackIDs.SetNumUninitialized(NumReferences);
			for(int32 i=0; i<NumReferences; ++i)
			{
				ar.Read(e.TrackIDs[i]);
			}
		}
	}

	const TArray<FMP4BoxTREF::FEntry>& FMP4BoxTREF::GetEntries()
	{
		ParseIfRequired();
		return Parsed->Entries;
	}
	TArray<FMP4BoxTREF::FEntry> FMP4BoxTREF::GetEntriesOfType(uint32 InType)
	{
		ParseIfRequired();
		TArray<FMP4BoxTREF::FEntry> Entries;
		for(auto& e : Parsed->Entries)
		{
			if (e.Type == InType)
			{
				Entries.Emplace(e);
			}
		}
		return Entries;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxELST::FParsed
	{
		uint32 Flags = 0;
		uint8 Version = 0;
		TArray<FEntry> Entries;
	};
	void FMP4BoxELST::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		ar.ReadVersionAndFlags(Parsed->Version, Parsed->Flags);
		uint32 NumEntries;
		ar.Read(NumEntries);
		Parsed->Entries.AddDefaulted((int32) NumEntries);
		if (Parsed->Version == 1)
		{
			for(uint32 i=0; i<NumEntries; ++i)
			{
				FEntry& e(Parsed->Entries[i]);
				ar.Read(e.edit_duration);
				ar.Read(e.media_time);
				ar.Read(e.media_rate_integer);
				ar.Read(e.media_rate_fraction);
			}
		}
		else
		{
			uint32 edit_duration;
			int32 media_time;
			for(uint32 i=0; i<NumEntries; ++i)
			{
				FEntry& e(Parsed->Entries[i]);
				ar.Read(edit_duration);
				e.edit_duration = edit_duration;
				ar.Read(media_time);
				e.media_time = media_time;
				ar.Read(e.media_rate_integer);
				ar.Read(e.media_rate_fraction);
			}
		}
	}

	const TArray<FMP4BoxELST::FEntry>& FMP4BoxELST::GetEntries()
	{
		ParseIfRequired();
		return Parsed->Entries;
	}
	bool FMP4BoxELST::RepeatEdits()
	{
		ParseIfRequired();
		return !!(Parsed->Flags & 1);
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxMDHD::FParsed
	{
		FString LanguageCode639_2T;
		int64 Duration = 0;
		uint32 Flags = 0;
		uint32 Timescale = 0;
		uint8 Version = 0;
	};
	void FMP4BoxMDHD::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		ar.ReadVersionAndFlags(Parsed->Version, Parsed->Flags);
		if (Parsed->Version == 1)
		{
			ar.SkipBytes(16);				// `creation_time` and `modification_time`
			ar.Read(Parsed->Timescale);
			ar.Read(Parsed->Duration);
			Parsed->Duration = Parsed->Duration < 0 ? TNumericLimits<int64>::Max() : Parsed->Duration;
		}
		else
		{
			ar.SkipBytes(8);				// `creation_time` and `modification_time`
			uint32 duration;
			ar.Read(Parsed->Timescale);
			ar.Read(duration);
			Parsed->Duration = duration;
		}

		uint16 Value16;
		ar.Read(Value16);
		char Language[3] { (char)(0x60 + ((Value16 & 0x7c00) >> 10)), (char)(0x60 + ((Value16 & 0x03e0) >> 5)), (char)(0x60 + (Value16 & 0x001f)) };
		Parsed->LanguageCode639_2T = FString::ConstructFromPtrSize(Language, UE_ARRAY_COUNT(Language));
	}

	MP4Utilities::FFractionalTime FMP4BoxMDHD::GetDuration()
	{
		ParseIfRequired();
		return MP4Utilities::FFractionalTime((int64) Parsed->Duration, Parsed->Timescale);
	}
	uint32 FMP4BoxMDHD::GetTimescale()
	{
		ParseIfRequired();
		return Parsed->Timescale;
	}


	FString FMP4BoxMDHD::GetLanguageCode639_2T()
	{
		ParseIfRequired();
		return Parsed->LanguageCode639_2T;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxHDLR::FParsed
	{
		FString HandlerName;
		uint32 Reserved[3] {};
		uint32 HandlerType = 0;
		uint32 Flags = 0;
		uint8 Version = 0;
	};
	void FMP4BoxHDLR::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		ar.ReadVersionAndFlags(Parsed->Version, Parsed->Flags);
		ar.SkipBytes(4);				// `pre_defined`
		ar.Read(Parsed->HandlerType);
		ar.Read(Parsed->Reserved[0]);
		ar.Read(Parsed->Reserved[1]);
		ar.Read(Parsed->Reserved[2]);
		ar.ReadStringUTF8(Parsed->HandlerName, ar.GetNumBytesRemaining());
	}

	uint32 FMP4BoxHDLR::GetHandlerType()
	{
		ParseIfRequired();
		return Parsed->HandlerType;
	}
	FString FMP4BoxHDLR::GetHandlerName()
	{
		ParseIfRequired();
		return Parsed->HandlerName;
	}
	uint32 FMP4BoxHDLR::GetReservedValue(int32 InIndex0to2)
	{
		ParseIfRequired();
		return InIndex0to2 >= 0 && InIndex0to2 <= 2 ? Parsed->Reserved[InIndex0to2] : 0;
	}


/****************************************************************************************************************************************************/

	struct FMP4BoxELNG::FParsed
	{
		FString LanguageTag;
		uint32 Flags = 0;
		uint8 Version = 0;
	};
	void FMP4BoxELNG::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		ar.ReadVersionAndFlags(Parsed->Version, Parsed->Flags);
		ar.ReadStringUTF8(Parsed->LanguageTag, ar.GetNumBytesRemaining());
	}

	FString FMP4BoxELNG::GetLanguageTag()
	{
		ParseIfRequired();
		return Parsed->LanguageTag;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxSTSD::FParsed
	{
		FMP4BoxSampleEntry::ESampleType SampleType = FMP4BoxSampleEntry::ESampleType::Unsupported;
		uint32 Flags = 0;
		uint8 Version = 0;
	};
	void FMP4BoxSTSD::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		ar.ReadVersionAndFlags(Parsed->Version, Parsed->Flags);

		// In order to parse the sample entry we need to know the media handler type.
		// Structurally this `stsd` box must be contained in a path `mdia`->`minf`->`stbl`->`stsd`, with
		// the required `hdlr` box being given under `mdia`, so we need to get our enclosing `mdia` box first.
		auto MDIABox = FindParentBox<FMP4BoxMDIA>(MP4Utilities::MakeBoxAtom('m','d','i','a'));
		if (!MDIABox.IsValid())
		{
			UE_LOGF(LogMP4Boxes, Error, "Could not find the parent `mdia` box to parse this `stsd` box!");
			return;
		}
		auto HDLRBox = MDIABox->FindBoxRecursive<FMP4BoxHDLR>(MP4Utilities::MakeBoxAtom('h','d','l','r'), 1);
		if (!HDLRBox.IsValid())
		{
			// If a handler is missing for whatever reason we could look for a `vmhd`, `smhd`, etc. box under the `minf` box.
			UE_LOGF(LogMP4Boxes, Error, "Could not find the corresponding `hdlr` box required to parse this `stsd` box!");
			return;
		}
		switch(HDLRBox->GetHandlerType())
		{
			case MP4Utilities::MakeBoxAtom('v','i','d','e'):
			{
				Parsed->SampleType = FMP4BoxSampleEntry::ESampleType::Video;
				break;
			}
			case MP4Utilities::MakeBoxAtom('s','o','u','n'):
			{
				Parsed->SampleType = FMP4BoxSampleEntry::ESampleType::Audio;
				break;
			}
			case MP4Utilities::MakeBoxAtom('t','m','c','d'):
			{
				Parsed->SampleType = FMP4BoxSampleEntry::ESampleType::QTFFTimecode;
				break;
			}
			case MP4Utilities::MakeBoxAtom('m','e','t','a'):
			{
				Parsed->SampleType = FMP4BoxSampleEntry::ESampleType::TimedMetadata;
				break;
			}
			case MP4Utilities::MakeBoxAtom('t','e','x','t'):
			{
				Parsed->SampleType = FMP4BoxSampleEntry::ESampleType::TimedText;
				break;
			}
			case MP4Utilities::MakeBoxAtom('s','b','t','l'):
			case MP4Utilities::MakeBoxAtom('s','u','b','t'):
			{
				Parsed->SampleType = FMP4BoxSampleEntry::ESampleType::Subtitles;
				break;
			}
			default:
			{
				Parsed->SampleType = FMP4BoxSampleEntry::ESampleType::Unsupported;
				break;
			}
		}
		// Unsupported sample type. No need to continue parsing.
		if (Parsed->SampleType == FMP4BoxSampleEntry::ESampleType::Unsupported)
		{
			return;
		}
		uint32 EntryCount;
		ar.Read(EntryCount);
		for(uint32 ne=0; ne<EntryCount; ++ne)
		{
			MP4Utilities::FMP4BoxInfo bi;
			if (!ar.ParseIntoBoxInfo(bi, BoxInfo.Offset + BoxInfo.DataOffset + ar.GetCurrentOffset()))
			{
				UE_LOGF(LogMP4Boxes, Error, "Failed to parse `stsd` box!");
				return;
			}
			TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> SampleEntry;
			switch(Parsed->SampleType)
			{
				case FMP4BoxSampleEntry::ESampleType::Video:
				{
					SampleEntry = FMP4BoxVisualSampleEntry::Create(AsWeak(), bi);
					break;
				}
				case FMP4BoxSampleEntry::ESampleType::Audio:
				{
					SampleEntry = FMP4BoxAudioSampleEntry::Create(AsWeak(), bi);
					break;
				}
				case FMP4BoxSampleEntry::ESampleType::QTFFTimecode:
				{
					SampleEntry = FMP4BoxQTFFTimecodeSampleEntry::Create(AsWeak(), bi);
					break;
				}
				case FMP4BoxSampleEntry::ESampleType::Subtitles:
				case FMP4BoxSampleEntry::ESampleType::TimedText:
				{
					switch(bi.Type)
					{
						case MP4Utilities::MakeBoxAtom('t','x','3','g'):
						{
							SampleEntry = FMP4BoxTX3GSampleEntry::Create(AsWeak(), bi);
							break;
						}
						case MP4Utilities::MakeBoxAtom('s','t','p','p'):
						{
							SampleEntry = FMP4BoxXMLSubtitleSampleEntry::Create(AsWeak(), bi);
							break;
						}
						case MP4Utilities::MakeBoxAtom('w','v','t','t'):
						{
							SampleEntry = FMP4BoxWVTTSampleEntry::Create(AsWeak(), bi);
							break;
						}
						default:
						{
							//check(!"TODO");
							// Ignored for now.
							break;
						}
					}
					break;
				}
				case FMP4BoxSampleEntry::ESampleType::TimedMetadata:
				{
					// Ignored for now.
					break;
				}
			}
			AddChildBox(MoveTemp(SampleEntry));
		}
	}

	uint8 FMP4BoxSTSD::GetBoxVersion()
	{
		ParseIfRequired();
		return Parsed->Version;
	}

	FMP4BoxSampleEntry::ESampleType FMP4BoxSTSD::GetSampleType()
	{
		ParseIfRequired();
		return Parsed->SampleType;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxSTSS::FParsed
	{
		TArray<uint32> Entries;
		uint32 Flags = 0;
		uint8 Version = 0;
	};
	void FMP4BoxSTSS::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		ar.ReadVersionAndFlags(Parsed->Version, Parsed->Flags);
		uint32 NumEntries;
		ar.Read(NumEntries);
		Parsed->Entries.AddUninitialized(NumEntries);
		for(uint32 i=0; i<NumEntries; ++i)
		{
			ar.Read(Parsed->Entries[i]);
		}
	}

	const TArray<uint32>& FMP4BoxSTSS::GetEntries()
	{
		ParseIfRequired();
		return Parsed->Entries;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxSDTP::FParsed
	{
		TConstArrayView<uint8> Entries;
		uint32 Flags = 0;
		uint8 Version = 0;
	};
	void FMP4BoxSDTP::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		ar.ReadVersionAndFlags(Parsed->Version, Parsed->Flags);
		uint32 NumEntries;
		ar.Read(NumEntries);
		Parsed->Entries = MakeConstArrayView(ar.GetCurrentDataPointer(), NumEntries);
	}

	TConstArrayView<uint8> FMP4BoxSDTP::GetEntries()
	{
		ParseIfRequired();
		return Parsed->Entries;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxVisualSampleEntry::FParsed
	{
		FString CompressorName;
		uint8 CompressorNameData[32] {};
		uint16 DataReferenceIndex = 0;
		uint16 Width = 0;
		uint16 Height = 0;
		uint16 FrameCount = 0;
		uint16 Depth = 0;
	};
	void FMP4BoxVisualSampleEntry::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		// Members of the generic `SampleEntry` class
		ar.SkipBytes(6);
		ar.Read(Parsed->DataReferenceIndex);
		// VisualSampleEntry follows
		ar.SkipBytes(16);		// `pre_defined`, `reserved`, `pre_defined`
		ar.Read(Parsed->Width);
		ar.Read(Parsed->Height);
		ar.SkipBytes(12);		// `horizresolution`, `vertresolution`, `reserved`
		ar.Read(Parsed->FrameCount);
		ar.ReadBytes(Parsed->CompressorNameData, 32);
		ar.Read(Parsed->Depth);
		ar.SkipBytes(2);		// `pre_defined`
		if (Parsed->CompressorNameData[0] && Parsed->CompressorNameData[0] < 32)
		{
			Parsed->CompressorName = FString::ConstructFromPtrSize(reinterpret_cast<const UTF8CHAR*>(Parsed->CompressorNameData + 1), Parsed->CompressorNameData[0]);
		}
		// There can now be additional boxes following, most notably `pasp` and `clap`.
		ProcessBoxChildrenRecursively(ar, BoxInfo);
	}

	uint16 FMP4BoxVisualSampleEntry::GetDataReferenceIndex()
	{
		ParseIfRequired();
		return Parsed->DataReferenceIndex;
	}
	uint16 FMP4BoxVisualSampleEntry::GetWidth()
	{
		ParseIfRequired();
		return Parsed->Width;
	}
	uint16 FMP4BoxVisualSampleEntry::GetHeight()
	{
		ParseIfRequired();
		return Parsed->Height;
	}
	uint16 FMP4BoxVisualSampleEntry::GetFrameCount()
	{
		ParseIfRequired();
		return Parsed->FrameCount;
	}
	uint16 FMP4BoxVisualSampleEntry::GetDepth()
	{
		ParseIfRequired();
		return Parsed->Depth;
	}
	FString FMP4BoxVisualSampleEntry::GetCompressorName()
	{
		ParseIfRequired();
		return Parsed->CompressorName;
	}


/****************************************************************************************************************************************************/

	struct FMP4BoxAudioSampleEntry::FParsed
	{
		uint32 SampleRate = 0;
		uint16 DataReferenceIndex = 0;
		uint16 Version = 0;
		uint16 ChannelCount = 0;
		uint16 SampleSize = 0;

		// QuickTime compatibility
		uint32 ConstBytesPerAudioPacket = 0;
		uint32 ConstLPCMFramesPerAudioPacket = 0;
		int32 FormatSpecificFlags = 0;
	};
	void FMP4BoxAudioSampleEntry::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		// Members of the generic `SampleEntry` class
		ar.SkipBytes(6);
		ar.Read(Parsed->DataReferenceIndex);
		// AudioSampleEntry follows
		ar.Read(Parsed->Version);
		ar.SkipBytes(6);		// `reserved`

		// The audio sample entry can be version 0 or version 1.
		// In ISO/IEC 14496-12 a version 1 sample is required to be inside a version 1 'stsd' box while in QuickTime a
		// version 1 sample was allowed in a version 0 'stsd'. The version 1 sample is not identical between ISO and QT
		// so we do some special handling based on version numbers.
		if (Parsed->Version == 0 || Parsed->Version == 1)
		{
			ar.Read(Parsed->ChannelCount);
			ar.Read(Parsed->SampleSize);
			ar.SkipBytes(4);	// `pre_defined`, `reserved`
			ar.Read(Parsed->SampleRate);
			// Handle a special case of version 1 QuickTime box. In ISO/IEC 14496-12 the SampleRate must be set to 0x00010000 (1 Hz).
			// If this is not the case we assume this to be a QuickTime box which adds 4 additional fields *before* any optional boxes.
			if (Parsed->Version == 1 && Parsed->SampleRate != (1U << 16))
			{
				// Which version is the enclosing `stsd` box?
				auto STSDBox = FindParentBox<FMP4BoxSTSD>(MP4Utilities::MakeBoxAtom('s','t','s','d'));
				if (!STSDBox.IsValid())
				{
					UE_LOGF(LogMP4Boxes, Error, "Could not find the parent `stsd` box to parse this AudioSampleEntry!");
					return;
				}
				if (STSDBox->GetBoxVersion() == 0)
				{
					// Assume QuickTime and skip the following elements
					ar.SkipBytes(4 * 4);	// Samples per packet; Bytes per packet; Bytes per frame; Bytes per Sample
				}
			}
			// The sample rate is stored in the upper 16 bits only. The lower 16 bits are 0.
			Parsed->SampleRate >>= 16;
		}
		// As defined by: https://developer.apple.com/documentation/quicktime-file-format/sound_sample_description_version_2
		else if (Parsed->Version == 2)
		{
			int32 SizeOfStructOnly = 0;
			int32 NumChannels = 0;
			int32 ConstBitsPerChannel = 0;
			uint32 Value32 = 0;
			uint16 Value16 = 0;
			union Int64toDouble
			{
				uint64 ui64;
				double dbl64;
			};
			Int64toDouble SampleRateCnv;
			SampleRateCnv.ui64 = 0;
			ar.Read(Value16);					// always 3
			ar.Read(Value16);					// always 16
			ar.Read(Value16);					// always 0xfffe
			ar.Read(Value16);					// always 0
			ar.Read(Value32);					// always 65536
			ar.Read(SizeOfStructOnly);
			ar.Read(SampleRateCnv.ui64);
			ar.Read(NumChannels);
			ar.Read(Value32);					// always 0x7f000000
			ar.Read(ConstBitsPerChannel);

			ar.Read(Parsed->FormatSpecificFlags);
			ar.Read(Parsed->ConstBytesPerAudioPacket);
			ar.Read(Parsed->ConstLPCMFramesPerAudioPacket);

			Parsed->SampleRate = (uint32) SampleRateCnv.dbl64;
			Parsed->ChannelCount = (uint16) NumChannels;
			Parsed->SampleSize = (uint16) ConstBitsPerChannel;
		}

		// There can now be additional boxes following, like `esds`, `chnl`, `dmix`, `btrt`, etc.
		ProcessBoxChildrenRecursively(ar, BoxInfo);

		// Check if there is an `srat` box to get the 32 bit sample rate from.
		auto SRATBox = FindBoxRecursive<FMP4BoxSRAT>(MP4Utilities::MakeBoxAtom('s','r','a','t'), 0);
		if (SRATBox.IsValid())
		{
			Parsed->SampleRate = SRATBox->GetSampleRate();
		}
	}

	uint16 FMP4BoxAudioSampleEntry::GetDataReferenceIndex()
	{
		ParseIfRequired();
		return Parsed->DataReferenceIndex;
	}
	uint32 FMP4BoxAudioSampleEntry::GetSampleRate()
	{
		ParseIfRequired();
		return Parsed->SampleRate;
	}
	int32 FMP4BoxAudioSampleEntry::GetChannelCount()
	{
		ParseIfRequired();
		return Parsed->ChannelCount;
	}
	int32 FMP4BoxAudioSampleEntry::GetSampleSize()
	{
		ParseIfRequired();
		return Parsed->SampleSize;
	}
	bool FMP4BoxAudioSampleEntry::HaveFormatSpecificFlags()
	{
		ParseIfRequired();
		return Parsed->Version == 2;
	}
	int32 FMP4BoxAudioSampleEntry::GetFormatSpecificFlags()
	{
		ParseIfRequired();
		return Parsed->FormatSpecificFlags;
	}
	uint32 FMP4BoxAudioSampleEntry::GetConstBytesPerAudioPacket()
	{
		ParseIfRequired();
		return Parsed->ConstBytesPerAudioPacket;
	}
	uint32 FMP4BoxAudioSampleEntry::GetConstLPCMFramesPerAudioPacket()
	{
		ParseIfRequired();
		return Parsed->ConstLPCMFramesPerAudioPacket;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxXMLSubtitleSampleEntry::FParsed
	{
		ESampleType SampleType = ESampleType::Subtitles;
		uint16 DataReferenceIndex = 0;
	};
	void FMP4BoxXMLSubtitleSampleEntry::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		// Members of the generic `SampleEntry` class
		ar.SkipBytes(6);
		ar.Read(Parsed->DataReferenceIndex);
/*
	typedefs:
		utf8list: null-terminated list of space-separated UTF-8 strings
		utf8string: UTF-8 string as defined in IETF RFC 3629, null-terminated.
	box members:
		utf8list namespace;
		utf8list schema_location; // optional
		utf8list auxiliary_mime_types;
		// optional, required if auxiliary resources are present

		// TX3GSampleEntry follows
		ar.SkipBytes(4);		// `reserved`
		ar.Read(Parsed->Flags);
		ar.Read(Parsed->Timescale);
		ar.Read(Parsed->FrameDuration);
		ar.Read(Parsed->NumberOfFrames);
		ar.SkipBytes(1);		// `reserved`
		// There can now be additional boxes following.
		ProcessBoxChildrenRecursively(ar, BoxInfo);
*/

		uint32 HandlerType = 0;
		auto MDIABox = FindParentBox<FMP4BoxMDIA>(MP4Utilities::MakeBoxAtom('m','d','i','a'));
		if (MDIABox.IsValid())
		{
			auto HDLRBox = MDIABox->FindBoxRecursive<FMP4BoxHDLR>(MP4Utilities::MakeBoxAtom('h','d','l','r'), 1);
			HandlerType = HDLRBox.IsValid() ? HDLRBox->GetHandlerType() : 0;
		}
		Parsed->SampleType = HandlerType == MP4Utilities::MakeBoxAtom('t','e','x','t') ? ESampleType::TimedText : ESampleType::Subtitles;
	}

	FMP4BoxSampleEntry::ESampleType FMP4BoxXMLSubtitleSampleEntry::GetSampleType()
	{
		ParseIfRequired();
		return Parsed->SampleType;
	}

	uint16 FMP4BoxXMLSubtitleSampleEntry::GetDataReferenceIndex()
	{
		ParseIfRequired();
		return Parsed->DataReferenceIndex;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxWVTTSampleEntry::FParsed
	{
		ESampleType SampleType = ESampleType::Subtitles;
		uint16 DataReferenceIndex = 0;
	};
	void FMP4BoxWVTTSampleEntry::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		// Members of the generic `SampleEntry` class
		ar.SkipBytes(6);
		ar.Read(Parsed->DataReferenceIndex);
		/*
			These boxes follow
				WebVTTConfigurationBox config;
				WebVTTSourceLabelBox label; // recommended
				MPEG4BitRateBox (); // optional
		*/
		ProcessBoxChildrenRecursively(ar, BoxInfo);

		uint32 HandlerType = 0;
		auto MDIABox = FindParentBox<FMP4BoxMDIA>(MP4Utilities::MakeBoxAtom('m','d','i','a'));
		if (MDIABox.IsValid())
		{
			auto HDLRBox = MDIABox->FindBoxRecursive<FMP4BoxHDLR>(MP4Utilities::MakeBoxAtom('h','d','l','r'), 1);
			HandlerType = HDLRBox.IsValid() ? HDLRBox->GetHandlerType() : 0;
		}
		Parsed->SampleType = HandlerType == MP4Utilities::MakeBoxAtom('t','e','x','t') ? ESampleType::TimedText : ESampleType::Subtitles;
	}

	FMP4BoxSampleEntry::ESampleType FMP4BoxWVTTSampleEntry::GetSampleType()
	{
		ParseIfRequired();
		return Parsed->SampleType;
	}

	uint16 FMP4BoxWVTTSampleEntry::GetDataReferenceIndex()
	{
		ParseIfRequired();
		return Parsed->DataReferenceIndex;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxQTFFTimecodeSampleEntry::FParsed
	{
		uint32 Flags = 0;
		uint32 Timescale = 0;
		uint32 FrameDuration = 0;
		uint16 DataReferenceIndex = 0;
		uint8 NumberOfFrames = 0;
	};
	void FMP4BoxQTFFTimecodeSampleEntry::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		// Members of the generic `SampleEntry` class
		ar.SkipBytes(6);
		ar.Read(Parsed->DataReferenceIndex);
		// QTFFTimecodeSampleEntry follows
		ar.SkipBytes(4);		// `reserved`
		ar.Read(Parsed->Flags);
		ar.Read(Parsed->Timescale);
		ar.Read(Parsed->FrameDuration);
		ar.Read(Parsed->NumberOfFrames);
		ar.SkipBytes(1);		// `reserved`
		// There can now be additional boxes following.
		ProcessBoxChildrenRecursively(ar, BoxInfo);
	}

	uint16 FMP4BoxQTFFTimecodeSampleEntry::GetDataReferenceIndex()
	{
		ParseIfRequired();
		return Parsed->DataReferenceIndex;
	}
	uint32 FMP4BoxQTFFTimecodeSampleEntry::GetFlags()
	{
		ParseIfRequired();
		return Parsed->Flags;
	}
	uint32 FMP4BoxQTFFTimecodeSampleEntry::GetTimescale()
	{
		ParseIfRequired();
		return Parsed->Timescale;
	}
	uint32 FMP4BoxQTFFTimecodeSampleEntry::GetFrameDuration()
	{
		ParseIfRequired();
		return Parsed->FrameDuration;
	}
	uint32 FMP4BoxQTFFTimecodeSampleEntry::GetNumberOfFrames()
	{
		ParseIfRequired();
		return Parsed->NumberOfFrames;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxTX3GSampleEntry::FParsed
	{
		ESampleType SampleType = ESampleType::Subtitles;
		uint16 DataReferenceIndex = 0;
	};
	void FMP4BoxTX3GSampleEntry::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		// Members of the generic `SampleEntry` class
		ar.SkipBytes(6);
		ar.Read(Parsed->DataReferenceIndex);
/*
		// TX3GSampleEntry follows
		ar.SkipBytes(4);		// `reserved`
		ar.Read(Parsed->Flags);
		ar.Read(Parsed->Timescale);
		ar.Read(Parsed->FrameDuration);
		ar.Read(Parsed->NumberOfFrames);
		ar.SkipBytes(1);		// `reserved`
		// There can now be additional boxes following.
		ProcessBoxChildrenRecursively(ar, BoxInfo);
*/
		uint32 HandlerType = 0;
		auto MDIABox = FindParentBox<FMP4BoxMDIA>(MP4Utilities::MakeBoxAtom('m','d','i','a'));
		if (MDIABox.IsValid())
		{
			auto HDLRBox = MDIABox->FindBoxRecursive<FMP4BoxHDLR>(MP4Utilities::MakeBoxAtom('h','d','l','r'), 1);
			HandlerType = HDLRBox.IsValid() ? HDLRBox->GetHandlerType() : 0;
		}
		Parsed->SampleType = HandlerType == MP4Utilities::MakeBoxAtom('t','e','x','t') ? ESampleType::TimedText : ESampleType::Subtitles;
	}

	FMP4BoxSampleEntry::ESampleType FMP4BoxTX3GSampleEntry::GetSampleType()
	{
		ParseIfRequired();
		return Parsed->SampleType;
	}

	uint16 FMP4BoxTX3GSampleEntry::GetDataReferenceIndex()
	{
		ParseIfRequired();
		return Parsed->DataReferenceIndex;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxAVCC::FParsed
	{
		TConstArrayView<uint8> AVCDecoderConfigurationRecord;
	};
	void FMP4BoxAVCC::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		Parsed->AVCDecoderConfigurationRecord = MakeConstArrayView(ar.GetCurrentDataPointer(), ar.GetNumBytesRemaining());
	}

	TConstArrayView<uint8> FMP4BoxAVCC::GetAVCDecoderConfigurationRecord()
	{
		ParseIfRequired();
		return Parsed->AVCDecoderConfigurationRecord;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxHVCC::FParsed
	{
		TConstArrayView<uint8> HEVCDecoderConfigurationRecord;
	};
	void FMP4BoxHVCC::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		Parsed->HEVCDecoderConfigurationRecord = MakeConstArrayView(ar.GetCurrentDataPointer(), ar.GetNumBytesRemaining());
	}

	TConstArrayView<uint8> FMP4BoxHVCC::GetHEVCDecoderConfigurationRecord()
	{
		ParseIfRequired();
		return Parsed->HEVCDecoderConfigurationRecord;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxVPCC::FParsed
	{
		TConstArrayView<uint8> VPCodecConfigurationRecord;
		uint32 Flags = 0;
		uint8 Version = 0;
	};
	void FMP4BoxVPCC::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		ar.ReadVersionAndFlags(Parsed->Version, Parsed->Flags);
		Parsed->VPCodecConfigurationRecord = MakeConstArrayView(ar.GetCurrentDataPointer(), ar.GetNumBytesRemaining());
	}

	TConstArrayView<uint8> FMP4BoxVPCC::GetVPCodecConfigurationRecord()
	{
		ParseIfRequired();
		return Parsed->VPCodecConfigurationRecord;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxAV1C::FParsed
	{
		TConstArrayView<uint8> AV1CodecConfigurationRecord;
	};
	void FMP4BoxAV1C::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		Parsed->AV1CodecConfigurationRecord = MakeConstArrayView(ar.GetCurrentDataPointer(), ar.GetNumBytesRemaining());
	}

	TConstArrayView<uint8> FMP4BoxAV1C::GetAV1CodecConfigurationRecord()
	{
		ParseIfRequired();
		return Parsed->AV1CodecConfigurationRecord;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxAPVC::FParsed
	{
		TConstArrayView<uint8> APVCodecConfigurationRecord;
	};
	void FMP4BoxAPVC::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		Parsed->APVCodecConfigurationRecord = MakeConstArrayView(ar.GetCurrentDataPointer(), ar.GetNumBytesRemaining());
	}

	TConstArrayView<uint8> FMP4BoxAPVC::GetAVPCCodecConfigurationRecord()
	{
		ParseIfRequired();
		return Parsed->APVCodecConfigurationRecord;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxDVCC::FParsed
	{
		TConstArrayView<uint8> DOVIDecoderConfigurationRecord;
	};
	void FMP4BoxDVCC::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		Parsed->DOVIDecoderConfigurationRecord = MakeConstArrayView(ar.GetCurrentDataPointer(), ar.GetNumBytesRemaining());
	}

	TConstArrayView<uint8> FMP4BoxDVCC::GetDOVIDecoderConfigurationRecord()
	{
		ParseIfRequired();
		return Parsed->DOVIDecoderConfigurationRecord;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxDAC3::FParsed
	{
		TConstArrayView<uint8> AC3SpecificBox;
	};
	void FMP4BoxDAC3::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		Parsed->AC3SpecificBox = MakeConstArrayView(ar.GetCurrentDataPointer(), ar.GetNumBytesRemaining());
	}

	TConstArrayView<uint8> FMP4BoxDAC3::GetAC3SpecificBox()
	{
		ParseIfRequired();
		return Parsed->AC3SpecificBox;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxDEC3::FParsed
	{
		TConstArrayView<uint8> EC3SpecificBox;
	};
	void FMP4BoxDEC3::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		Parsed->EC3SpecificBox = MakeConstArrayView(ar.GetCurrentDataPointer(), ar.GetNumBytesRemaining());
	}

	TConstArrayView<uint8> FMP4BoxDEC3::GetEC3SpecificBox()
	{
		ParseIfRequired();
		return Parsed->EC3SpecificBox;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxDFLA::FParsed
	{
		TConstArrayView<uint8> FLACSpecificBox;
	};
	void FMP4BoxDFLA::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		Parsed->FLACSpecificBox = MakeConstArrayView(ar.GetCurrentDataPointer(), ar.GetNumBytesRemaining());
	}

	TConstArrayView<uint8> FMP4BoxDFLA::GetFLACSpecificBox()
	{
		ParseIfRequired();
		return Parsed->FLACSpecificBox;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxDOPS::FParsed
	{
		TConstArrayView<uint8> OpusSpecificBox;
	};
	void FMP4BoxDOPS::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		Parsed->OpusSpecificBox = MakeConstArrayView(ar.GetCurrentDataPointer(), ar.GetNumBytesRemaining());
	}

	TConstArrayView<uint8> FMP4BoxDOPS::GetOpusSpecificBox()
	{
		ParseIfRequired();
		return Parsed->OpusSpecificBox;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxIODS::FParsed
	{
		TConstArrayView<uint8> InitialObjectDescriptor;
	};
	void FMP4BoxIODS::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		Parsed->InitialObjectDescriptor = MakeConstArrayView(ar.GetCurrentDataPointer(), ar.GetNumBytesRemaining());
	}

	TConstArrayView<uint8> FMP4BoxIODS::GetObjectDescriptor()
	{
		ParseIfRequired();
		return Parsed->InitialObjectDescriptor;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxESDS::FParsed
	{
		TConstArrayView<uint8> ESDescriptor;
		uint32 Flags = 0;
		uint8 Version = 0;
	};
	void FMP4BoxESDS::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		ar.ReadVersionAndFlags(Parsed->Version, Parsed->Flags);
		Parsed->ESDescriptor = MakeConstArrayView(ar.GetCurrentDataPointer(), ar.GetNumBytesRemaining());
	}

	TConstArrayView<uint8> FMP4BoxESDS::GetESDescriptor()
	{
		ParseIfRequired();
		return Parsed->ESDescriptor;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxCOLR::FParsed
	{
		FColorNCLX ColorNCLX;
		EColorType ColorType = EColorType::Unsupported;
	};
	void FMP4BoxCOLR::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		uint32 colour_type;
		ar.Read(colour_type);
		if (colour_type == MP4Utilities::MakeBoxAtom('n','c','l','c') || colour_type == MP4Utilities::MakeBoxAtom('n','c','l','x'))
		{
			Parsed->ColorType = EColorType::nclx;
			uint16 colour_primaries, transfer_characteristics, matrix_coefficients;
			ar.Read(colour_primaries);
			ar.Read(transfer_characteristics);
			ar.Read(matrix_coefficients);
			Parsed->ColorNCLX.colour_primaries = colour_primaries;
			Parsed->ColorNCLX.transfer_characteristics = transfer_characteristics;
			Parsed->ColorNCLX.matrix_coefficients = matrix_coefficients;
			if (colour_type == MP4Utilities::MakeBoxAtom('n','c','l','x'))
			{
				uint8 full_range_flag;
				ar.Read(full_range_flag);
				Parsed->ColorNCLX.full_range_flag = full_range_flag >> 7;
			}
		}
	}

	FMP4BoxCOLR::EColorType FMP4BoxCOLR::GetColorType()
	{
		ParseIfRequired();
		return Parsed->ColorType;
	}
	const FMP4BoxCOLR::FColorNCLX& FMP4BoxCOLR::GetColorNCLX()
	{
		ParseIfRequired();
		return Parsed->ColorNCLX;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxBTRT::FParsed
	{
		uint32 BufferSizeDB = 0;
		uint32 MaxBitrate = 0;
		uint32 AverageBitrate = 0;
	};
	void FMP4BoxBTRT::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		ar.Read(Parsed->BufferSizeDB);
		ar.Read(Parsed->MaxBitrate);
		ar.Read(Parsed->AverageBitrate);
	}

	uint32 FMP4BoxBTRT::GetBufferSizeDB()
	{
		ParseIfRequired();
		return Parsed->BufferSizeDB;
	}
	uint32 FMP4BoxBTRT::GetMaxBitrate()
	{
		ParseIfRequired();
		return Parsed->MaxBitrate;
	}
	uint32 FMP4BoxBTRT::GetAverageBitrate()
	{
		ParseIfRequired();
		return Parsed->AverageBitrate;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxSRAT::FParsed
	{
		uint32 SampleRate = 0;
		uint32 Flags = 0;
		uint8 Version = 0;
	};
	void FMP4BoxSRAT::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		ar.ReadVersionAndFlags(Parsed->Version, Parsed->Flags);
		ar.Read(Parsed->SampleRate);
	}

	uint32 FMP4BoxSRAT::GetSampleRate()
	{
		ParseIfRequired();
		return Parsed->SampleRate;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxSTTS::FParsed
	{
		TArray<FEntry> Entries;
		int64 TotalDuration = 0;
		uint32 NumTotalSamples = 0;
		uint32 Flags = 0;
		uint8 Version = 0;
	};
	void FMP4BoxSTTS::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		ar.ReadVersionAndFlags(Parsed->Version, Parsed->Flags);
		uint32 entry_count;
		ar.Read(entry_count);
		Parsed->Entries.SetNumUninitialized(entry_count);
		for(uint32 i=0; i<entry_count; ++i)
		{
			ar.Read(Parsed->Entries[i].sample_count);
			ar.Read(Parsed->Entries[i].sample_delta);
			Parsed->NumTotalSamples += Parsed->Entries[i].sample_count;
			Parsed->TotalDuration += Parsed->Entries[i].sample_count * Parsed->Entries[i].sample_delta;
		}
	}

	uint32 FMP4BoxSTTS::GetNumTotalSamples()
	{
		ParseIfRequired();
		return Parsed->NumTotalSamples;
	}
	int64 FMP4BoxSTTS::GetTotalDuration()
	{
		ParseIfRequired();
		return Parsed->TotalDuration;
	}
	const TArray<FMP4BoxSTTS::FEntry>& FMP4BoxSTTS::GetEntries()
	{
		ParseIfRequired();
		return Parsed->Entries;
	}
	bool FMP4BoxSTTS::HasConstantSampleDuration()
	{
		ParseIfRequired();
		return Parsed->Entries.Num() == 1;
	}
	int64 FMP4BoxSTTS::GetConstantSampleDuration()
	{
		ParseIfRequired();
		return Parsed->Entries.Num() == 1 ? Parsed->Entries[0].sample_delta : 0;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxCTTS::FParsed
	{
		TArray<FEntry> Entries;
		uint32 NumTotalSamples = 0;
		uint32 Flags = 0;
		uint8 Version = 0;
	};
	void FMP4BoxCTTS::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		ar.ReadVersionAndFlags(Parsed->Version, Parsed->Flags);
		uint32 entry_count;
		ar.Read(entry_count);
	/*
		Note: We read this box unconditionally as if it were version 1 using signed composition offsets.
			    This addresses the issue that under the QuickTime brand this box is always using signed integers
				even in version 0.
				It stands to reason that no video will actually require unsigned values >0x7fffffff even if using
				HNS as timescale for which 0x7fffffff would still give about 214 seconds...

		if (Parsed->Version == 0)
		{
			Parsed->Entries.SetNumUninitialized(entry_count);
			uint32 Value32;
			for(uint32 i=0; i<entry_count; ++i)
			{
				ar.Read(Parsed->Entries[i].sample_count);
				Parsed->NumTotalSamples += Parsed->Entries[i].sample_count;
				ar.Read(Value32);
				Parsed->Entries[i].sample_offset = Value32;
			}
		}
		else if (Parsed->Version == 1)
	*/
		{
			Parsed->Entries.SetNumUninitialized(entry_count);
			int32 Value32;
			for(uint32 i=0; i<entry_count; ++i)
			{
				ar.Read(Parsed->Entries[i].sample_count);
				Parsed->NumTotalSamples += Parsed->Entries[i].sample_count;
				ar.Read(Value32);
				Parsed->Entries[i].sample_offset = Value32;
			}
		}
	}

	uint8 FMP4BoxCTTS::GetBoxVersion()
	{
		ParseIfRequired();
		return Parsed->Version;
	}
	uint32 FMP4BoxCTTS::GetNumTotalSamples()
	{
		ParseIfRequired();
		return Parsed->NumTotalSamples;
	}
	const TArray<FMP4BoxCTTS::FEntry>& FMP4BoxCTTS::GetEntries()
	{
		ParseIfRequired();
		return Parsed->Entries;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxSTSC::FParsed
	{
		TArray<FEntry> Entries;
		uint32 Flags = 0;
		uint8 Version = 0;
	};
	void FMP4BoxSTSC::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		ar.ReadVersionAndFlags(Parsed->Version, Parsed->Flags);
		uint32 entry_count;
		ar.Read(entry_count);
		Parsed->Entries.SetNumUninitialized(entry_count);
		static_assert(sizeof(FEntry::first_chunk) == 4);
		static_assert(sizeof(FEntry::samples_per_chunk) == 4);
		static_assert(sizeof(FEntry::sample_description_index) == 4);
		for(uint32 i=0; i<entry_count; ++i)
		{
			ar.Read(Parsed->Entries[i].first_chunk);
			ar.Read(Parsed->Entries[i].samples_per_chunk);
			ar.Read(Parsed->Entries[i].sample_description_index);
		}
	}

	const TArray<FMP4BoxSTSC::FEntry>& FMP4BoxSTSC::GetEntries()
	{
		ParseIfRequired();
		return Parsed->Entries;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxSTSZ::FParsed
	{
		const uint32* SizeArray32 = nullptr;
		const uint16* SizeArray16 = nullptr;
		const uint8* SizeArray8 = nullptr;
		const uint8* SizeArray4 = nullptr;
		uint32 NumberOfSamples = 0;
		uint32 ConstantSampleSize = 0;
		uint32 FieldSize = 0;
		uint32 Flags = 0;
		uint8 Version = 0;
	};
	void FMP4BoxSTSZ::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		ar.ReadVersionAndFlags(Parsed->Version, Parsed->Flags);
		if (BoxInfo.Type == MP4Utilities::MakeBoxAtom('s','t','s','z'))
		{
			ar.Read(Parsed->ConstantSampleSize);
			ar.Read(Parsed->NumberOfSamples);
			if (Parsed->ConstantSampleSize == 0)
			{
				Parsed->SizeArray32 = (const uint32*) ar.GetCurrentDataPointer();
			}
		}
		else
		{
			ar.Read(Parsed->FieldSize);
			check((Parsed->FieldSize >> 8) == 0);
			check(Parsed->FieldSize == 4 || Parsed->FieldSize == 8 || Parsed->FieldSize == 16);
			ar.Read(Parsed->NumberOfSamples);
			if (Parsed->FieldSize == 16)
			{
				Parsed->SizeArray16 = (const uint16*) ar.GetCurrentDataPointer();
			}
			else if (Parsed->FieldSize == 8)
			{
				Parsed->SizeArray8 = (const uint8*) ar.GetCurrentDataPointer();
			}
			else if (Parsed->FieldSize == 4)
			{
				Parsed->SizeArray4 = (const uint8*) ar.GetCurrentDataPointer();
			}
		}
	}

	uint32 FMP4BoxSTSZ::GetNumberOfSamples()
	{
		ParseIfRequired();
		return Parsed->NumberOfSamples;
	}
	uint32 FMP4BoxSTSZ::GetSizeOfSample(uint32 InIndex)
	{
		ParseIfRequired();
		if (InIndex >= Parsed->NumberOfSamples)
		{
			return 0;
		}
		if (Parsed->ConstantSampleSize)
		{
			return Parsed->ConstantSampleSize;
		}
		else if (Parsed->SizeArray32)
		{
			return MP4Utilities::GetFromBigEndian(Parsed->SizeArray32[InIndex]);
		}
		else if (Parsed->SizeArray16)
		{
			return MP4Utilities::GetFromBigEndian(Parsed->SizeArray16[InIndex]);
		}
		else if (Parsed->SizeArray8)
		{
			return Parsed->SizeArray8[InIndex];
		}
		else if (Parsed->SizeArray4)
		{
			uint8 sz = Parsed->SizeArray4[InIndex >> 1];
			return (InIndex & 1) == 0 ? (sz >> 4) : (sz & 15);
		}
		return 0;
	}
	bool FMP4BoxSTSZ::HasConstantSampleSize()
	{
		ParseIfRequired();
		return Parsed->ConstantSampleSize != 0;
	}
	uint32 FMP4BoxSTSZ::GetConstantSampleSize()
	{
		ParseIfRequired();
		return Parsed->ConstantSampleSize;
	}



/****************************************************************************************************************************************************/

	struct FMP4BoxSTCO::FParsed
	{
		const uint32* SizeArray32 = nullptr;
		const uint64* SizeArray64 = nullptr;
		uint32 NumberOfEntries = 0;
		uint32 Flags = 0;
		uint8 Version = 0;
	};
	void FMP4BoxSTCO::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		ar.ReadVersionAndFlags(Parsed->Version, Parsed->Flags);
		ar.Read(Parsed->NumberOfEntries);
		if (BoxInfo.Type == MP4Utilities::MakeBoxAtom('c','o','6','4'))
		{
			Parsed->SizeArray64 = (const uint64*) ar.GetCurrentDataPointer();
		}
		else
		{
			Parsed->SizeArray32 = (const uint32*) ar.GetCurrentDataPointer();
		}
	}

	uint32 FMP4BoxSTCO::GetNumberOfEntries()
	{
		ParseIfRequired();
		return Parsed->NumberOfEntries;
	}
	uint64 FMP4BoxSTCO::GetChunkOffset(uint32 InIndex)
	{
		ParseIfRequired();
		return InIndex < Parsed->NumberOfEntries ? Parsed->SizeArray32 ? MP4Utilities::GetFromBigEndian(Parsed->SizeArray32[InIndex]) : MP4Utilities::GetFromBigEndian(Parsed->SizeArray64[InIndex]) : 0;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxSUBS::FParsed
	{
		TArray<FMP4BoxSUBS::FEntry> Entries;
		uint32 Flags = 0;
		uint8 Version = 0;
	};
	void FMP4BoxSUBS::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		ar.ReadVersionAndFlags(Parsed->Version, Parsed->Flags);
		uint32 entry_count;
		ar.Read(entry_count);
		Parsed->Entries.SetNum(entry_count);
		uint32 SampleIndex = 0;
		for(uint32 i=0; i<entry_count; ++i)
		{
			uint32 sample_delta;
			uint16 subsample_count;
			ar.Read(sample_delta);
			ar.Read(subsample_count);
			SampleIndex += sample_delta;
			check(i || sample_delta);	// the first delta must not be zero
			Parsed->Entries[i].FirstSampleNumber = SampleIndex;
			if (subsample_count)
			{
				uint32 subsample_offset = 0;
				Parsed->Entries[i].SubSamples.Reserve(subsample_count);
				for(int32 j=0; j<subsample_count; ++j)
				{
					FMP4BoxSUBS::FSubsample ss;
					if (Parsed->Version == 1)
					{
						ar.Read(ss.subsample_size);
					}
					else
					{
						uint16 sssize;
						ar.Read(sssize);
						ss.subsample_size = sssize;
					}
					ar.Read(ss.subsample_priority);
					ar.Read(ss.discardable);
					ar.Read(ss.codec_specific_parameters);
					// Calculate the offset into the sample for convenience.
					ss.subsample_offset = subsample_offset;
					subsample_offset += ss.subsample_size;
					Parsed->Entries[i].SubSamples.Emplace(MoveTemp(ss));
				}
			}
		}
	}

	uint32 FMP4BoxSUBS::GetFlags()
	{
		ParseIfRequired();
		return Parsed->Flags;
	}

	const TArray<FMP4BoxSUBS::FEntry>& FMP4BoxSUBS::GetEntries()
	{
		ParseIfRequired();
		return Parsed->Entries;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxSAIZ::FParsed
	{
		TConstArrayView<uint8> SampleInfoSizes;
		uint32 AuxInfoType = 0;
		uint32 AuxInfoTypeParameter = 0;
		uint32 SampleCount = 0;
		uint32 Flags = 0;
		uint8 Version = 0;
		uint8 DefaultSampleInfoSize = 0;
	};
	void FMP4BoxSAIZ::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		ar.ReadVersionAndFlags(Parsed->Version, Parsed->Flags);
		if ((Parsed->Flags & 1) != 0)
		{
			ar.Read(Parsed->AuxInfoType);
			ar.Read(Parsed->AuxInfoTypeParameter);
		}
		ar.Read(Parsed->DefaultSampleInfoSize);
		ar.Read(Parsed->SampleCount);
		if (Parsed->DefaultSampleInfoSize == 0 && Parsed->SampleCount != 0)
		{
			Parsed->SampleInfoSizes = MakeConstArrayView(ar.GetCurrentDataPointer(), Parsed->SampleCount);
		}
	}

	bool FMP4BoxSAIZ::HasAuxInfoType()
	{
		ParseIfRequired();
		return !!(Parsed->Flags & 1);
	}
	uint32 FMP4BoxSAIZ::GetAuxInfoType()
	{
		ParseIfRequired();
		return Parsed->AuxInfoType;
	}
	uint32 FMP4BoxSAIZ::GetAuxInfoTypeParameter()
	{
		ParseIfRequired();
		return Parsed->AuxInfoTypeParameter;
	}
	uint32 FMP4BoxSAIZ::GetSampleCount()
	{
		ParseIfRequired();
		return Parsed->SampleCount;
	}
	uint8 FMP4BoxSAIZ::GetSampleInfoSize(uint32 InForSampleNumber)
	{
		ParseIfRequired();
		return Parsed->DefaultSampleInfoSize ? Parsed->DefaultSampleInfoSize :
			   InForSampleNumber < Parsed->SampleCount ? Parsed->SampleInfoSizes[InForSampleNumber] : 0;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxSAIO::FParsed
	{
		const uint64* Offsets64 = nullptr;
		const uint32* Offsets32 = nullptr;
		uint32 AuxInfoType = 0;
		uint32 AuxInfoTypeParameter = 0;
		uint32 EntryCount = 0;
		uint32 Flags = 0;
		uint8 Version = 0;
	};
	void FMP4BoxSAIO::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		ar.ReadVersionAndFlags(Parsed->Version, Parsed->Flags);
		if ((Parsed->Flags & 1) != 0)
		{
			ar.Read(Parsed->AuxInfoType);
			ar.Read(Parsed->AuxInfoTypeParameter);
		}
		ar.Read(Parsed->EntryCount);
		// Remember a 32 or 64 bit pointer to where the offsets are.
		// When accessing them remember that they are in big endian and need to be swapped on return value!
		if (Parsed->Version == 0)
		{
			Parsed->Offsets32 = (const uint32*)ar.GetCurrentDataPointer();
		}
		else
		{
			Parsed->Offsets64 = (const uint64*)ar.GetCurrentDataPointer();
		}
	}

	bool FMP4BoxSAIO::HasAuxInfoType()
	{
		ParseIfRequired();
		return !!(Parsed->Flags & 1);
	}
	uint32 FMP4BoxSAIO::GetAuxInfoType()
	{
		ParseIfRequired();
		return Parsed->AuxInfoType;
	}
	uint32 FMP4BoxSAIO::GetAuxInfoTypeParameter()
	{
		ParseIfRequired();
		return Parsed->AuxInfoTypeParameter;
	}
	uint32 FMP4BoxSAIO::GetEntryCount()
	{
		ParseIfRequired();
		return Parsed->EntryCount;
	}
	uint64 FMP4BoxSAIO::GetEntry(uint32 InForEntryNumber)
	{
		ParseIfRequired();
		return InForEntryNumber >= Parsed->EntryCount ? 0 :
				Parsed->Offsets32 ? MP4Utilities::GetFromBigEndian(Parsed->Offsets32[InForEntryNumber]) :
				Parsed->Offsets64 ? MP4Utilities::GetFromBigEndian(Parsed->Offsets64[InForEntryNumber]) : 0;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxSGPD::FParsed
	{
		TArray<TConstArrayView<uint8>> GroupDescriptionEntries;
		uint32 GroupingType = 0;
		uint32 DefaultLength = 0;
		uint32 DefaultGroupDescriptionIndex = 0;
		uint32 Flags = 0;
		uint8 Version = 0;
	#if !UE_BUILD_SHIPPING
		char Name[5]{0};
	#endif
	};
	void FMP4BoxSGPD::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		ar.ReadVersionAndFlags(Parsed->Version, Parsed->Flags);
		if (Parsed->Version == 0)
		{
			UE_LOGF(LogMP4Boxes, Warning, "Version 0 of the `sgpd` box is deprecated. Box will be ignored.");
			return;
		}
		ar.Read(Parsed->GroupingType);
	#if !UE_BUILD_SHIPPING
		Parsed->Name[0] = (char) ((Parsed->GroupingType >> 24) & 255);
		Parsed->Name[1] = (char) ((Parsed->GroupingType >> 16) & 255);
		Parsed->Name[2] = (char) ((Parsed->GroupingType >>  8) & 255);
		Parsed->Name[3] = (char) ((Parsed->GroupingType >>  0) & 255);
	#endif
		if (Parsed->Version >= 1)
		{
			ar.Read(Parsed->DefaultLength);
		}
		if (Parsed->Version >= 2)
		{
			ar.Read(Parsed->DefaultGroupDescriptionIndex);
		}
		uint32 entry_count;
		ar.Read(entry_count);
		Parsed->GroupDescriptionEntries.SetNum(entry_count);
		for(uint32 i=0; i<entry_count; ++i)
		{
			uint32 description_length = Parsed->DefaultLength;
			if (Parsed->Version == 1 && description_length == 0)
			{
				ar.Read(description_length);
			}
			Parsed->GroupDescriptionEntries[i] = MakeConstArrayView(ar.GetCurrentDataPointer(), description_length);
		}
	}

	uint32 FMP4BoxSGPD::GetGroupingType()
	{
		ParseIfRequired();
		return Parsed->GroupingType;
	}
	const TArray<TConstArrayView<uint8>>& FMP4BoxSGPD::GetGroupDescriptionEntries()
	{
		ParseIfRequired();
		return Parsed->GroupDescriptionEntries;
	}
	uint32 FMP4BoxSGPD::GetDefaultGroupDescriptionIndex()
	{
		ParseIfRequired();
		return Parsed->DefaultGroupDescriptionIndex;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxSBGP::FParsed
	{
		TArray<FEntry> Entries;
		uint32 GroupingType = 0;
		uint32 GroupingTypeParameter = 0;
		uint32 NumTotalSamples = 0;
		uint32 Flags = 0;
		uint8 Version = 0;
	#if !UE_BUILD_SHIPPING
		char Name[5]{0};
	#endif
	};
	void FMP4BoxSBGP::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		ar.ReadVersionAndFlags(Parsed->Version, Parsed->Flags);
		ar.Read(Parsed->GroupingType);
	#if !UE_BUILD_SHIPPING
		Parsed->Name[0] = (char) ((Parsed->GroupingType >> 24) & 255);
		Parsed->Name[1] = (char) ((Parsed->GroupingType >> 16) & 255);
		Parsed->Name[2] = (char) ((Parsed->GroupingType >>  8) & 255);
		Parsed->Name[3] = (char) ((Parsed->GroupingType >>  0) & 255);
	#endif
		if (Parsed->Version == 1)
		{
			ar.Read(Parsed->GroupingTypeParameter);
		}
		uint32 entry_count;
		ar.Read(entry_count);
		Parsed->Entries.SetNum(entry_count);
		static_assert(sizeof(FEntry::sample_count) == 4);
		static_assert(sizeof(FEntry::group_description_index) == 4);
		for(uint32 i=0; i<entry_count; ++i)
		{
			ar.Read(Parsed->Entries[i].sample_count);
			Parsed->NumTotalSamples += Parsed->Entries[i].sample_count;
			ar.Read(Parsed->Entries[i].group_description_index);
		}
	}

	uint32 FMP4BoxSBGP::GetGroupingType()
	{
		ParseIfRequired();
		return Parsed->GroupingType;
	}
	uint32 FMP4BoxSBGP::GetGroupingTypeParameter()
	{
		ParseIfRequired();
		return Parsed->GroupingTypeParameter;
	}
	const TArray<FMP4BoxSBGP::FEntry>& FMP4BoxSBGP::GetEntries()
	{
		ParseIfRequired();
		return Parsed->Entries;
	}
	uint32 FMP4BoxSBGP::GetNumTotalSamples()
	{
		ParseIfRequired();
		return Parsed->NumTotalSamples;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxMEHD::FParsed
	{
		uint64 FragmentDuration = 0;
		uint32 Flags = 0;
		uint8 Version = 0;
	};
	void FMP4BoxMEHD::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		ar.ReadVersionAndFlags(Parsed->Version, Parsed->Flags);
		if (Parsed->Version == 1)
		{
			ar.Read(Parsed->FragmentDuration);
			if (Parsed->FragmentDuration > 0x7fffffffffffffffULL)
			{
				Parsed->FragmentDuration = 0x7fffffffffffffffULL;
				UE_LOGF(LogMP4Boxes, Warning, "`mehd` version 1 fragment duration cannot be represented as a signed value. Why is it so large?");
			}
		}
		else
		{
			uint32 Value32;
			ar.Read(Value32);
			Parsed->FragmentDuration = Value32;
		}
	}

	int64 FMP4BoxMEHD::GetFragmentDuration()
	{
		ParseIfRequired();
		return (int64) Parsed->FragmentDuration;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxTREX::FParsed
	{
		uint32 TrackID = 0;
		uint32 DefaultSampleDescriptionIndex = 0;
		uint32 DefaultSampleDuration = 0;
		uint32 DefaultSampleSize = 0;
		uint32 DefaultSampleFlags = 0;
		uint32 Flags = 0;
		uint8 Version = 0;
	};
	void FMP4BoxTREX::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		ar.ReadVersionAndFlags(Parsed->Version, Parsed->Flags);
		ar.Read(Parsed->TrackID);
		ar.Read(Parsed->DefaultSampleDescriptionIndex);
		ar.Read(Parsed->DefaultSampleDuration);
		ar.Read(Parsed->DefaultSampleSize);
		ar.Read(Parsed->DefaultSampleFlags);
	}

	uint32 FMP4BoxTREX::GetTrackID()
	{
		ParseIfRequired();
		return Parsed->TrackID;
	}
	uint32 FMP4BoxTREX::GetDefaultSampleDescriptionIndex()
	{
		ParseIfRequired();
		return Parsed->DefaultSampleDescriptionIndex;
	}
	uint32 FMP4BoxTREX::GetDefaultSampleDuration()
	{
		ParseIfRequired();
		return Parsed->DefaultSampleDuration;
	}
	uint32 FMP4BoxTREX::GetDefaultSampleSize()
	{
		ParseIfRequired();
		return Parsed->DefaultSampleSize;
	}
	uint32 FMP4BoxTREX::GetDefaultSampleFlags()
	{
		ParseIfRequired();
		return Parsed->DefaultSampleFlags;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxMFHD::FParsed
	{
		uint32 SequenceNumber = 0;
		uint32 Flags = 0;
		uint8 Version = 0;
	};
	void FMP4BoxMFHD::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		ar.ReadVersionAndFlags(Parsed->Version, Parsed->Flags);
		ar.Read(Parsed->SequenceNumber);
	}

	uint32 FMP4BoxMFHD::GetSequenceNumber()
	{
		ParseIfRequired();
		return Parsed->SequenceNumber;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxTFHD::FParsed
	{
		uint64 BaseDataOffset = 0;
		uint32 TrackID = 0;
		uint32 SampleDescriptionIndex = 0;
		uint32 DefaultSampleDuration = 0;
		uint32 DefaultSampleSize = 0;
		uint32 DefaultSampleFlags = 0;
		uint32 Flags = 0;
		uint8 Version = 0;
	};
	void FMP4BoxTFHD::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		ar.ReadVersionAndFlags(Parsed->Version, Parsed->Flags);
		ar.Read(Parsed->TrackID);
		if ((Parsed->Flags & 1) != 0)
		{
			ar.Read(Parsed->BaseDataOffset);			// base_data_offset
		}
		if ((Parsed->Flags & 2) != 0)
		{
			ar.Read(Parsed->SampleDescriptionIndex);	// sample_description_index
		}
		if ((Parsed->Flags & 8) != 0)
		{
			ar.Read(Parsed->DefaultSampleDuration);		// default_sample_duration
		}
		if ((Parsed->Flags & 16) != 0)
		{
			ar.Read(Parsed->DefaultSampleSize);			// default_sample_size
		}
		if ((Parsed->Flags & 32) != 0)
		{
			ar.Read(Parsed->DefaultSampleFlags);		// default_sample_flags
		}
	}

	uint32 FMP4BoxTFHD::GetTrackID()
	{
		ParseIfRequired();
		return Parsed->TrackID;
	}

	bool FMP4BoxTFHD::HasBaseDataOffset()
	{
		ParseIfRequired();
		return (Parsed->Flags & 0x000001) != 0;
	}
	uint64 FMP4BoxTFHD::GetBaseDataOffset()
	{
		ParseIfRequired();
		return Parsed->BaseDataOffset;
	}

	bool FMP4BoxTFHD::HasSampleDescriptionIndex()
	{
		ParseIfRequired();
		return (Parsed->Flags & 0x000002) != 0;
	}
	uint32 FMP4BoxTFHD::GetSampleDescriptionIndex()
	{
		ParseIfRequired();
		return Parsed->SampleDescriptionIndex;
	}

	bool FMP4BoxTFHD::HasDefaultSampleDuration()
	{
		ParseIfRequired();
		return (Parsed->Flags & 0x000008) != 0;
	}
	uint32 FMP4BoxTFHD::GetDefaultSampleDuration()
	{
		ParseIfRequired();
		return Parsed->DefaultSampleDuration;
	}

	bool FMP4BoxTFHD::HasDefaultSampleSize()
	{
		ParseIfRequired();
		return (Parsed->Flags & 0x000010) != 0;
	}
	uint32 FMP4BoxTFHD::GetDefaultSampleSize()
	{
		ParseIfRequired();
		return Parsed->DefaultSampleSize;
	}

	bool FMP4BoxTFHD::HasDefaultSampleFlags()
	{
		ParseIfRequired();
		return (Parsed->Flags & 0x000020) != 0;
	}
	uint32 FMP4BoxTFHD::GetDefaultSampleFlags()
	{
		ParseIfRequired();
		return Parsed->DefaultSampleFlags;
	}

	bool FMP4BoxTFHD::IsDurationEmpty()
	{
		ParseIfRequired();
		return (Parsed->Flags & 0x010000) != 0;
	}

	bool FMP4BoxTFHD::IsMoofDefaultBase()
	{
		ParseIfRequired();
		return (Parsed->Flags & 0x020000) != 0;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxTFDT::FParsed
	{
		uint64 BaseMediaDecodeTime = 0;
		uint32 Flags = 0;
		uint8 Version = 0;
	};
	void FMP4BoxTFDT::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		ar.ReadVersionAndFlags(Parsed->Version, Parsed->Flags);
		if (Parsed->Version == 1)
		{
			ar.Read(Parsed->BaseMediaDecodeTime);
		}
		else
		{
			uint32 Value32;
			ar.Read(Value32);
			Parsed->BaseMediaDecodeTime = Value32;
		}
	}

	uint64 FMP4BoxTFDT::GetBaseMediaDecodeTime()
	{
		ParseIfRequired();
		return Parsed->BaseMediaDecodeTime;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxTRUN::FParsed
	{
		TArray<uint32> SampleDurations;
		TArray<uint32> SampleSizes;
		TArray<uint32> SampleFlags;
		TArray<int32> SampleCompositionTimeOffsets;
		int64 TotalDuration = 0;
		int64 TotalSampleSize = 0;
		uint32 SampleCount = 0;
		uint32 FirstSampleFlags = 0;
		uint32 Flags = 0;
		int32 DataOffset = 0;
		uint8 Version = 0;
	};
	void FMP4BoxTRUN::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		ar.ReadVersionAndFlags(Parsed->Version, Parsed->Flags);
		ar.Read(Parsed->SampleCount);
		if ((Parsed->Flags & 1) != 0)
		{
			ar.Read(Parsed->DataOffset);
		}
		if ((Parsed->Flags & 4) != 0)
		{
			ar.Read(Parsed->FirstSampleFlags);
		}
		if (Parsed->SampleCount)
		{
			if ((Parsed->Flags & 0x100) != 0)
			{
				Parsed->SampleDurations.SetNumUninitialized(Parsed->SampleCount);
			}
			if ((Parsed->Flags & 0x200) != 0)
			{
				Parsed->SampleSizes.SetNumUninitialized(Parsed->SampleCount);
			}
			if ((Parsed->Flags & 0x400) != 0)
			{
				Parsed->SampleFlags.SetNumUninitialized(Parsed->SampleCount);
			}
			if ((Parsed->Flags & 0x800) != 0)
			{
				Parsed->SampleCompositionTimeOffsets.SetNumUninitialized(Parsed->SampleCount);
			}
			for(uint32 i=0; i<Parsed->SampleCount; ++i)
			{
				if ((Parsed->Flags & 0x100) != 0)
				{
					uint32 dur;
					ar.Read(dur);
					Parsed->SampleDurations[i] = dur;
					Parsed->TotalDuration += dur;
				}
				if ((Parsed->Flags & 0x200) != 0)
				{
					uint32 sz;
					ar.Read(sz);
					Parsed->SampleSizes[i] = sz;
					Parsed->TotalSampleSize += sz;
				}
				if ((Parsed->Flags & 0x400) != 0)
				{
					ar.Read(Parsed->SampleFlags[i]);
				}
				if ((Parsed->Flags & 0x800) != 0)
				{
					if (Parsed->Version == 0)
					{
						uint32 Value32;
						ar.Read(Value32);
						/*
							Because we want to handle only signed time offsets we check if the value can actually be
							presented as such. If not then the first question would be why the value is that large,
							which could indicate a bad file. If that is legitimate then we would need to change
							our SampleCompositionTimeOffsets to be an int64 table.
						*/

						if (Value32 > 0x7fffffffU)
						{
							UE_LOGF(LogMP4Boxes, Warning, "`trun` version 0 time value cannot be represented as a signed value. Why is it so large?");
							Value32 = 0x7fffffffU;
						}
						Parsed->SampleCompositionTimeOffsets[i] = (int32)Value32;
					}
					else
					{
						ar.Read(Parsed->SampleCompositionTimeOffsets[i]);
					}
				}
			}
		}
	}

	uint32 FMP4BoxTRUN::GetNumberOfSamples()
	{
		ParseIfRequired();
		return Parsed->SampleCount;
	}
	bool FMP4BoxTRUN::HasSampleOffset()
	{
		ParseIfRequired();
		return (Parsed->Flags & 0x000001) != 0;
	}
	int32 FMP4BoxTRUN::GetSampleOffset()
	{
		ParseIfRequired();
		return Parsed->DataOffset;
	}
	bool FMP4BoxTRUN::HasFirstSampleFlags()
	{
		ParseIfRequired();
		return (Parsed->Flags & 0x000004) != 0;
	}
	uint32 FMP4BoxTRUN::GetFirstSampleFlags()
	{
		ParseIfRequired();
		return Parsed->FirstSampleFlags;
	}
	bool FMP4BoxTRUN::HasSampleDurations()
	{
		ParseIfRequired();
		return (Parsed->Flags & 0x000100) != 0;
	}
	const TArray<uint32>& FMP4BoxTRUN::GetSampleDurations()
	{
		ParseIfRequired();
		return Parsed->SampleDurations;
	}
	bool FMP4BoxTRUN::HasSampleSizes()
	{
		ParseIfRequired();
		return (Parsed->Flags & 0x000200) != 0;
	}
	const TArray<uint32>& FMP4BoxTRUN::GetSampleSizes()
	{
		ParseIfRequired();
		return Parsed->SampleSizes;
	}
	int64 FMP4BoxTRUN::GetTotalSampleSize()
	{
		ParseIfRequired();
		return Parsed->TotalSampleSize;
	}

	bool FMP4BoxTRUN::HasSampleFlags()
	{
		ParseIfRequired();
		return (Parsed->Flags & 0x000400) != 0;
	}
	const TArray<uint32>& FMP4BoxTRUN::GetSampleFlags()
	{
		ParseIfRequired();
		return Parsed->SampleFlags;
	}
	bool FMP4BoxTRUN::HasSampleCompositionTimeOffsets()
	{
		ParseIfRequired();
		return (Parsed->Flags & 0x000800) != 0;
	}
	const TArray<int32>& FMP4BoxTRUN::GetSampleCompositionTimeOffsets()
	{
		ParseIfRequired();
		return Parsed->SampleCompositionTimeOffsets;
	}
	int64 FMP4BoxTRUN::GetTotalDuration()
	{
		ParseIfRequired();
		return Parsed->TotalDuration;
	}

/****************************************************************************************************************************************************/
	struct FMP4BoxFRMA::FParsed
	{
		uint32 OriginalFormat = 0;
	};
	void FMP4BoxFRMA::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		ar.Read(Parsed->OriginalFormat);
	}

	uint32 FMP4BoxFRMA::GetOriginalFormat()
	{
		ParseIfRequired();
		return Parsed->OriginalFormat;
	}

/****************************************************************************************************************************************************/
	struct FMP4BoxSCHM::FParsed
	{
		FString SchemeURI;
		uint32 SchemeType = 0;
		uint32 SchemeVersion = 0;
		uint32 Flags = 0;
		uint8 Version = 0;
	};
	void FMP4BoxSCHM::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		ar.ReadVersionAndFlags(Parsed->Version, Parsed->Flags);
		ar.Read(Parsed->SchemeType);
		ar.Read(Parsed->SchemeVersion);
		if ((Parsed->Flags & 1) != 0)
		{
			ar.ReadNullTerminatedStringUTF8(Parsed->SchemeURI);
		}
	}

	uint32 FMP4BoxSCHM::GetSchemeType()
	{
		ParseIfRequired();
		return Parsed->SchemeType;
	}

	uint32 FMP4BoxSCHM::GetSchemeVersion()
	{
		ParseIfRequired();
		return Parsed->SchemeVersion;
	}

/****************************************************************************************************************************************************/
	struct FMP4BoxPSSH::FParsed
	{
		TConstArrayView<uint8> SystemID;
		TArray<TConstArrayView<uint8>> KIDs;
		TConstArrayView<uint8> Data;
		uint32 Flags = 0;
		uint8 Version = 0;
	};
	void FMP4BoxPSSH::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		ar.ReadVersionAndFlags(Parsed->Version, Parsed->Flags);
		Parsed->SystemID = MakeConstArrayView(ar.GetCurrentDataPointer(), 16);
		ar.SkipBytes(16);
		if (Parsed->Version > 0)
		{
			uint32 KID_count;
			ar.Read(KID_count);
			Parsed->KIDs.SetNum(KID_count);
			for(uint32 i=0; i<KID_count; ++i)
			{
				Parsed->KIDs[i] = MakeConstArrayView(ar.GetCurrentDataPointer(), 16);
				ar.SkipBytes(16);
			}
		}
		uint32 DataSize;
		ar.Read(DataSize);
		Parsed->Data = MakeConstArrayView(ar.GetCurrentDataPointer(), DataSize);
	}

	TConstArrayView<uint8> FMP4BoxPSSH::GetSystemID()
	{
		ParseIfRequired();
		return Parsed->SystemID;
	}
	const TArray<TConstArrayView<uint8>>& FMP4BoxPSSH::GetKIDs()
	{
		ParseIfRequired();
		return Parsed->KIDs;
	}
	TConstArrayView<uint8> FMP4BoxPSSH::GetData()
	{
		ParseIfRequired();
		return Parsed->Data;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxTENC::FParsed
	{
		TConstArrayView<uint8> DefaultKID;
		TConstArrayView<uint8> DefaultConstantIV;
		uint32 Flags = 0;
		uint8 Version = 0;
		uint8 DefaultCryptByteBlock = 0;
		uint8 DefaultSkipByteBlock = 0;
		uint8 DefaultIsProtected = 0;
		uint8 DefaultPerSampleIVSize = 0;
	};
	void FMP4BoxTENC::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		ar.ReadVersionAndFlags(Parsed->Version, Parsed->Flags);
		if (Parsed->Version == 0)
		{
			ar.SkipBytes(2);			// `reserved`
		}
		else
		{
			ar.SkipBytes(1);			// `reserved`
			uint8 Value8;
			ar.Read(Value8);
			Parsed->DefaultCryptByteBlock = Value8 >> 4;
			Parsed->DefaultSkipByteBlock = Value8 & 0x0f;
		}
		ar.Read(Parsed->DefaultIsProtected);
		ar.Read(Parsed->DefaultPerSampleIVSize);
		Parsed->DefaultKID = MakeConstArrayView(ar.GetCurrentDataPointer(), 16);
		ar.SkipBytes(16);
		if (Parsed->DefaultIsProtected == 1 && Parsed->DefaultPerSampleIVSize == 0)
		{
			uint8 default_constant_IV_size;
			ar.Read(default_constant_IV_size);
			Parsed->DefaultConstantIV = MakeConstArrayView(ar.GetCurrentDataPointer(), default_constant_IV_size);
			ar.SkipBytes(default_constant_IV_size);
		}
	}

	bool FMP4BoxTENC::HasDefaultCryptBlockValues()
	{
		ParseIfRequired();
		return Parsed->Version != 0;
	}
	uint8 FMP4BoxTENC::GetDefaultCryptByteBlock()
	{
		ParseIfRequired();
		return Parsed->DefaultCryptByteBlock;
	}
	uint8 FMP4BoxTENC::GetDefaultSkipByteBlock()
	{
		ParseIfRequired();
		return Parsed->DefaultSkipByteBlock;
	}
	uint8 FMP4BoxTENC::GetDefaultIsProtected()
	{
		ParseIfRequired();
		return Parsed->DefaultIsProtected;
	}
	uint8 FMP4BoxTENC::GetDefaultPerSampleIVSize()
	{
		ParseIfRequired();
		return Parsed->DefaultPerSampleIVSize;
	}
	TConstArrayView<uint8> FMP4BoxTENC::GetDefaultKID()
	{
		ParseIfRequired();
		return Parsed->DefaultKID;
	}
	TConstArrayView<uint8> FMP4BoxTENC::GetDefaultConstantIV()
	{
		ParseIfRequired();
		return Parsed->DefaultConstantIV;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxSENC::FParsed
	{
		TArray<FEntry> Entries;
		uint32 Flags = 0;
		uint8 Version = 0;
	};
	void FMP4BoxSENC::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		ar.ReadVersionAndFlags(Parsed->Version, Parsed->Flags);
		auto TencBox = ParentTENCBox.Pin();
		if (TencBox)
		{
			// Check that we do not try to handle a pre-PIFF 1.3 box which has a different layout.
			if ((Parsed->Flags & 1) != 0)
			{
				UE_LOGF(LogMP4Boxes, Error, "The `senc` box is too old a version that is not supported");
				return;
			}

			/*
				Depending on the version of this box and the scheme being used we will need access to a
				number of other related boxes like `saiz`, `saio`, `seig` and possibly `ienc` and `iaux`.
				At the very least for box versions 0 and 2 the related `tenc` box is required.

				For the time being we only support version 0 where all samples are encrypted and a
				`seig` box is not necessary.
			*/
			if (Parsed->Version != 0)
			{
				UE_LOGF(LogMP4Boxes, Error, "At the moment only version 0 of the `senc` box is supported");
				return;
			}

			// We need the `Per_Sample_IV_Size` value from the `tenc` box.
			const uint8 Per_Sample_IV_Size = TencBox->GetDefaultPerSampleIVSize();

			const bool senc_use_subsamples = !!(Parsed->Flags & 2);
			const bool UseSubSampleEncryption = senc_use_subsamples /*&& Parsed->Version == 0*/;	// version already checked for above

			uint32 sample_count;
			ar.Read(sample_count);
			Parsed->Entries.SetNum(sample_count);
			for(uint32 i=0; i<sample_count; ++i)
			{
				FEntry& e = Parsed->Entries[i];
				/*
				if (Parsed->Version == 0)
				{
				*/
					if (Per_Sample_IV_Size)
					{
						e.IV = MakeConstArrayView(ar.GetCurrentDataPointer(), Per_Sample_IV_Size);
						ar.SkipBytes(Per_Sample_IV_Size);
					}
					else
					{
						// For convenience we set the contant IV from the `tenc` box here.
						e.IV = TencBox->GetDefaultConstantIV();
					}
					if (UseSubSampleEncryption)
					{
						uint16 subsample_count;
						ar.Read(subsample_count);
						e.SubSamples.SetNum(subsample_count);
						for(int32 k=0; k<subsample_count; ++k)
						{
							ar.Read(e.SubSamples[k].NumClearBytes);
							ar.Read(e.SubSamples[k].NumEncryptedBytes);
						}
					}
				/*
				}
				else if (Parsed->Version == 1 && isProtected)
				{
				}
				else if (Parsed->Version == 2 && isProtected)
				{
				}
				*/
			}
		}
		else
		{
			UE_LOGF(LogMP4Boxes, Error, "Cannot access a `senc` that has not been prepared with the corresponding `tenc`box");
		}
	}

	void FMP4BoxSENC::Prepare(TWeakPtr<FMP4BoxTENC, ESPMode::ThreadSafe> InTENCBox)
	{
		ParentTENCBox = InTENCBox;
		ParseIfRequired();
	}

	const TArray<FMP4BoxSENC::FEntry>& FMP4BoxSENC::GetEntries()
	{
		ParseIfRequired();
		return Parsed->Entries;
	}

/****************************************************************************************************************************************************/

	struct FMP4BoxSIDX::FParsed
	{
		FEntryList Entries;
		uint64 EarliestPresentationTime = 0;
		uint64 FirstOffset = 0;
		uint32 ReferenceID = 0;
		uint32 Timescale = 0;
		uint32 Flags = 0;
		uint8 Version = 0;
	};
	void FMP4BoxSIDX::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		ar.ReadVersionAndFlags(Parsed->Version, Parsed->Flags);
		ar.Read(Parsed->ReferenceID);
		ar.Read(Parsed->Timescale);
		if (Parsed->Version == 0)
		{
			uint32 Value32;
			ar.Read(Value32);
			Parsed->EarliestPresentationTime = Value32;
			ar.Read(Value32);
			Parsed->FirstOffset = Value32;
		}
		else
		{
			ar.Read(Parsed->EarliestPresentationTime);
			ar.Read(Parsed->FirstOffset);
		}
		ar.SkipBytes(2);		// `reserved`
		uint16 reference_count;
		ar.Read(reference_count);
		Parsed->Entries.Reserve(reference_count);
		static_assert(sizeof(FEntry::SubSegmentDuration) == 4);
		for(int32 i=0; i<reference_count; ++i)
		{
			FEntry e;
			uint32 ReferenceTypeAndSize;
			uint32 SAPStartAndTypeAndDeltaTime;
			ar.Read(ReferenceTypeAndSize);
			ar.Read(e.SubSegmentDuration);
			ar.Read(SAPStartAndTypeAndDeltaTime);
			e.IsReferenceType = ReferenceTypeAndSize >> 31;
			e.Size = ReferenceTypeAndSize & 0x7fffffff;
			e.StartsWithSAP  = SAPStartAndTypeAndDeltaTime >> 31;
			e.SAPType = (SAPStartAndTypeAndDeltaTime >> 28) & 7;
			e.SAPDeltaTime = SAPStartAndTypeAndDeltaTime & 0x0fffffffU;
			Parsed->Entries.AddElement(MoveTemp(e));
		}
	}

	uint32 FMP4BoxSIDX::GetReferenceID()
	{
		ParseIfRequired();
		return Parsed->ReferenceID;
	}
	uint32 FMP4BoxSIDX::GetTimescale()
	{
		ParseIfRequired();
		return Parsed->Timescale;
	}
	uint64 FMP4BoxSIDX::GetEarliestPresentationTime()
	{
		ParseIfRequired();
		return Parsed->EarliestPresentationTime;
	}
	uint64 FMP4BoxSIDX::GetFirstOffset()
	{
		ParseIfRequired();
		return Parsed->FirstOffset;
	}
	const FMP4BoxSIDX::FEntryList& FMP4BoxSIDX::GetEntries()
	{
		ParseIfRequired();
		return Parsed->Entries;
	}


/****************************************************************************************************************************************************/
	struct FMP4BoxMETA::FParsed
	{
		uint32 Flags = 0;
		uint8 Version = 0;
	};
	void FMP4BoxMETA::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);

		// The `meta` box is a full box under ISO/IEC 14496-12 and a basic box in QuickTime.
		// Some tools write a QT `meta` box even for ISO which is then missing the version number and flags!
		// To work around this issue we try to locate the mandatory `hdlr` box that needs to be the first
		// box and based on its position handle this as a basic or a full box.
		bool bIsFullBox = false;
		if (BoxInfo.Data.Num() >= 12)
		{
			bIsFullBox = BoxInfo.Data[8] == 'h' && BoxInfo.Data[9] == 'd' && BoxInfo.Data[10] == 'l' && BoxInfo.Data[11] == 'r';
		}
		if (bIsFullBox)
		{
			ar.ReadVersionAndFlags(Parsed->Version, Parsed->Flags);
		}

		while(ar.GetNumBytesRemaining() >= 8)
		{
			MP4Utilities::FMP4BoxInfo bi;
			if (!ar.ParseIntoBoxInfo(bi, BoxInfo.Offset + BoxInfo.DataOffset + ar.GetCurrentOffset()))
			{
				UE_LOGF(LogMP4Boxes, Error, "Failed to parse `meta` box!");
				return;
			}
			TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> ChildBox = FMP4BoxFactory::Get().Create(AsWeak(), bi);
			AddChildBox(MoveTemp(ChildBox));
			ar.SkipBytes(bi.Data.Num());
		}
	}

	TSharedPtr<FMP4BoxHDLR, ESPMode::ThreadSafe> FMP4BoxMETA::GetHandler()
	{
		ParseIfRequired();
		return FindBoxRecursive<FMP4BoxHDLR>(MP4Utilities::MakeBoxAtom('h','d','l','r'), 0);
	}

/****************************************************************************************************************************************************/
	struct FMP4BoxPRFT::FParsed
	{
		uint64 NTPTimestamp = 0;
		uint64 MediaTime = 0;
		uint32 ReferenceTrackID = 0;
		uint32 Flags = 0;
		uint8 Version = 0;
	};
	void FMP4BoxPRFT::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		ar.ReadVersionAndFlags(Parsed->Version, Parsed->Flags);
		ar.Read(Parsed->ReferenceTrackID);
		ar.Read(Parsed->NTPTimestamp);
		if (Parsed->Version == 0)
		{
			uint32 v;
			ar.Read(v);
			Parsed->MediaTime = v;
		}
		else
		{
			ar.Read(Parsed->MediaTime);
		}
	}

	uint32 FMP4BoxPRFT::GetFlags()
	{
		ParseIfRequired();
		return Parsed->Flags;
	}
	uint32 FMP4BoxPRFT::GetReferenceTrackID()
	{
		ParseIfRequired();
		return Parsed->ReferenceTrackID;
	}
	uint64 FMP4BoxPRFT::GetNTPTimestamp()
	{
		ParseIfRequired();
		return Parsed->NTPTimestamp;
	}
	uint64 FMP4BoxPRFT::GetMediaTime()
	{
		ParseIfRequired();
		return Parsed->MediaTime;
	}

/****************************************************************************************************************************************************/
	struct FMP4BoxEMSG::FParsed
	{
		TArray<uint8> MessageData;
		FString SchemeIdUri;
		FString Value;
		uint64 PresentationTimeOrDelta = 0;
		uint32 Timescale = 0;
		uint32 EventDuration = 0;
		uint32 ID = 0;
		uint32 Flags = 0;
		uint8 Version = 0;
	};
	void FMP4BoxEMSG::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		ar.ReadVersionAndFlags(Parsed->Version, Parsed->Flags);
		if (Parsed->Version == 0)
		{
			ar.ReadNullTerminatedStringUTF8(Parsed->SchemeIdUri);
			ar.ReadNullTerminatedStringUTF8(Parsed->Value);
			ar.Read(Parsed->Timescale);
			uint32 PresentationTimeDelta;
			ar.Read(PresentationTimeDelta);
			Parsed->PresentationTimeOrDelta = PresentationTimeDelta;
			ar.Read(Parsed->EventDuration);
			ar.Read(Parsed->ID);
		}
		else
		{
			ar.Read(Parsed->Timescale);
			ar.Read(Parsed->PresentationTimeOrDelta);
			ar.Read(Parsed->EventDuration);
			ar.Read(Parsed->ID);
			ar.ReadNullTerminatedStringUTF8(Parsed->SchemeIdUri);
			ar.ReadNullTerminatedStringUTF8(Parsed->Value);
		}
		if (ar.GetNumBytesRemaining())
		{
			Parsed->MessageData.SetNumUninitialized(ar.GetNumBytesRemaining());
			ar.ReadBytes(Parsed->MessageData.GetData(), ar.GetNumBytesRemaining());
		}
	}

	uint8 FMP4BoxEMSG::GetVersion()
	{
		ParseIfRequired();
		return Parsed->Version;
	}
	FString FMP4BoxEMSG::GetSchemeIdUri()
	{
		ParseIfRequired();
		return Parsed->SchemeIdUri;
	}
	FString FMP4BoxEMSG::GetValue()
	{
		ParseIfRequired();
		return Parsed->Value;
	}
	uint32 FMP4BoxEMSG::GetTimescale()
	{
		ParseIfRequired();
		return Parsed->Timescale;
	}
	uint32 FMP4BoxEMSG::GetPresentationTimeDelta()
	{
		ParseIfRequired();
		return (uint32)Parsed->PresentationTimeOrDelta;
	}
	uint64 FMP4BoxEMSG::GetPresentationTime()
	{
		ParseIfRequired();
		return Parsed->PresentationTimeOrDelta;
	}
	uint32 FMP4BoxEMSG::GetEventDuration()
	{
		ParseIfRequired();
		return Parsed->EventDuration;
	}
	uint32 FMP4BoxEMSG::GetID()
	{
		ParseIfRequired();
		return Parsed->ID;
	}
	TConstArrayView<uint8> FMP4BoxEMSG::GetMessageData()
	{
		ParseIfRequired();
		return Parsed->MessageData;
	}

/****************************************************************************************************************************************************/
#if 0 // TEMPLATE
	struct FMP4BoxCCCC::FParsed
	{
		uint32 Flags = 0;
		uint8 Version = 0;
	};
	void FMP4BoxCCCC::ParseIfRequired()
	{
		if (Parsed.IsValid())
		{
			return;
		}
		Parsed = MakePimpl<FParsed>();
		MP4Utilities::FMP4AtomReader ar(BoxInfo.Data);
		ar.ReadVersionAndFlags(Parsed->Version, Parsed->Flags);
		if (Parsed->Version == 1)
		{
		}
		else
		{
		}
	}

	bool FMP4BoxCCCC::XXXX()
	{
		ParseIfRequired();
		return
	}
#endif
/****************************************************************************************************************************************************/

	FMP4BoxFactory& FMP4BoxFactory::Get()
	{
		static FMP4BoxFactory Factory;
		return Factory;
	}

	FMP4BoxFactory::FMP4BoxFactory()
	{
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('f','t','y','p'), &FMP4BoxFTYP::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('s','t','y','p'), &FMP4BoxSTYP::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('m','o','o','v'), &FMP4BoxMOOV::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('m','v','h','d'), &FMP4BoxMVHD::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('t','r','a','k'), &FMP4BoxTRAK::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('t','k','h','d'), &FMP4BoxTKHD::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('t','r','e','f'), &FMP4BoxTREF::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('e','d','t','s'), &FMP4BoxEDTS::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('e','l','s','t'), &FMP4BoxELST::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('m','d','i','a'), &FMP4BoxMDIA::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('m','d','h','d'), &FMP4BoxMDHD::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('h','d','l','r'), &FMP4BoxHDLR::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('e','l','n','g'), &FMP4BoxELNG::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('m','i','n','f'), &FMP4BoxMINF::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('d','i','n','f'), &FMP4BoxDINF::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('d','r','e','f'), &FMP4BoxDREF::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('s','t','b','l'), &FMP4BoxSTBL::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('s','t','s','d'), &FMP4BoxSTSD::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('s','t','t','s'), &FMP4BoxSTTS::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('c','t','t','s'), &FMP4BoxCTTS::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('c','s','l','g'), &FMP4BoxCSLG::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('s','t','s','s'), &FMP4BoxSTSS::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('s','d','t','p'), &FMP4BoxSDTP::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('s','t','s','c'), &FMP4BoxSTSC::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('s','t','s','z'), &FMP4BoxSTSZ::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('s','t','z','2'), &FMP4BoxSTSZ::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('s','t','c','o'), &FMP4BoxSTCO::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('c','o','6','4'), &FMP4BoxSTCO::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('s','u','b','s'), &FMP4BoxSUBS::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('s','a','i','z'), &FMP4BoxSAIZ::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('s','a','i','o'), &FMP4BoxSAIO::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('s','g','p','d'), &FMP4BoxSGPD::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('s','b','g','p'), &FMP4BoxSBGP::Create);

		FactoryMap.Add(MP4Utilities::MakeBoxAtom('m','v','e','x'), &FMP4BoxMVEX::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('m','e','h','d'), &FMP4BoxMEHD::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('t','r','e','x'), &FMP4BoxTREX::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('p','r','f','t'), &FMP4BoxPRFT::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('l','e','v','a'), &FMP4BoxLEVA::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('m','o','o','f'), &FMP4BoxMOOF::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('m','f','h','d'), &FMP4BoxMFHD::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('t','r','a','f'), &FMP4BoxTRAF::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('t','f','h','d'), &FMP4BoxTFHD::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('t','r','u','n'), &FMP4BoxTRUN::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('m','f','r','a'), &FMP4BoxMFRA::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('t','f','r','a'), &FMP4BoxTFRA::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('m','f','r','o'), &FMP4BoxMFRO::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('t','f','d','t'), &FMP4BoxTFDT::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('s','i','d','x'), &FMP4BoxSIDX::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('s','s','i','x'), &FMP4BoxSSIX::Create);

		FactoryMap.Add(MP4Utilities::MakeBoxAtom('f','r','e','e'), &FMP4BoxFREE::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('s','k','i','p'), &FMP4BoxFREE::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('u','d','t','a'), &FMP4BoxUDTA::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('m','e','t','a'), &FMP4BoxMETA::Create);

		FactoryMap.Add(MP4Utilities::MakeBoxAtom('v','m','h','d'), &FMP4BoxVMHD::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('s','m','h','d'), &FMP4BoxSMHD::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('n','m','h','d'), &FMP4BoxNMHD::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('s','t','h','d'), &FMP4BoxSTHD::Create);

		FactoryMap.Add(MP4Utilities::MakeBoxAtom('g','m','h','d'), &FMP4BoxGMHD::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('t','a','p','t'), &FMP4BoxTAPT::Create);

		FactoryMap.Add(MP4Utilities::MakeBoxAtom('c','o','l','r'), &FMP4BoxCOLR::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('c','l','l','i'), &FMP4BoxCLLI::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('m','d','c','v'), &FMP4BoxMDCV::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('b','t','r','t'), &FMP4BoxBTRT::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('p','a','s','p'), &FMP4BoxPASP::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('s','r','a','t'), &FMP4BoxSRAT::Create);

		FactoryMap.Add(MP4Utilities::MakeBoxAtom('s','i','n','f'), &FMP4BoxSINF::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('f','r','m','a'), &FMP4BoxFRMA::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('s','c','h','m'), &FMP4BoxSCHM::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('s','c','h','i'), &FMP4BoxSCHI::Create);

		FactoryMap.Add(MP4Utilities::MakeBoxAtom('p','s','s','h'), &FMP4BoxPSSH::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('t','e','n','c'), &FMP4BoxTENC::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('s','e','n','c'), &FMP4BoxSENC::Create);

		FactoryMap.Add(MP4Utilities::MakeBoxAtom('e','m','s','g'), &FMP4BoxEMSG::Create);

		// `mdat` box is never parsed, we only handle it as a base box here to create the box tree.
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('m','d','a','t'), &FMP4BoxBase::Create);

//			FactoryMap.Add(MP4Utilities::MakeBoxAtom('u','r','l',' '), &FMP4BoxBase::Create);


		// Derived formats
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('a','v','c','C'), &FMP4BoxAVCC::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('h','v','c','C'), &FMP4BoxHVCC::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('d','v','c','C'), &FMP4BoxDVCC::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('d','v','v','C'), &FMP4BoxDVCC::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('d','v','w','C'), &FMP4BoxDVCC::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('d','a','c','3'), &FMP4BoxDAC3::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('d','e','c','3'), &FMP4BoxDEC3::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('i','o','d','s'), &FMP4BoxIODS::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('e','s','d','s'), &FMP4BoxESDS::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('w','a','v','e'), &FMP4BoxWAVE::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('d','f','L','a'), &FMP4BoxDFLA::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('d','O','p','s'), &FMP4BoxDOPS::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('v','p','c','C'), &FMP4BoxVPCC::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('a','v','1','C'), &FMP4BoxAV1C::Create);
		FactoryMap.Add(MP4Utilities::MakeBoxAtom('a','p','v','C'), &FMP4BoxAPVC::Create);
	}
	TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> FMP4BoxFactory::Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
	{
		if (auto Factory = FactoryMap.Find(InBoxInfo.Type))
		{
			return (*Factory)(InParent, InBoxInfo);
		}

		// Check for an `uuid` box. We know some of them and can handle them appropriately.
		if (InBoxInfo.Type == MP4Utilities::MakeBoxAtom('u','u','i','d'))
		{
			static const uint8 UUID_atom[16] { 0x00,0x00,0x00,0x00,0x00,0x11,0x00,0x10,0x80,0x00,0x00,0xAA,0x00,0x38,0x9B,0x71 };
			static const uint8 UUID_pssh[16] { 0xD0,0x8A,0x4F,0x18,0x10,0xF3,0x4A,0x82,0xB6,0xC8,0x32,0xD8,0xAB,0xA1,0x83,0xD3 };
			static const uint8 UUID_tenc[16] { 0x89,0x74,0xDB,0xCE,0x7B,0xE7,0x4C,0x51,0x84,0xF9,0x71,0x48,0xF9,0x88,0x25,0x54 };
			static const uint8 UUID_senc[16] { 0xA2,0x39,0x4F,0x52,0x5A,0x9B,0x4F,0x14,0xA2,0x44,0x6C,0x42,0x7C,0x64,0x8D,0xF4 };
			if (FMemory::Memcmp(InBoxInfo.UUID+4, UUID_atom+4, 12) == 0)
			{
				// This is not handled.
				UE_LOGF(LogMP4Boxes, Error, "A `uuid` box that uses the long form of a well known atom is not supported");
			}
			// Compare the UUID with known boxes.
			else if (FMemory::Memcmp(InBoxInfo.UUID, UUID_pssh, 16) == 0)
			{
				MP4Utilities::FMP4BoxInfo WellKnown(InBoxInfo);
				WellKnown.Type = MP4Utilities::MakeBoxAtom('p','s','s','h');
				return FMP4BoxPSSH::Create(InParent, WellKnown);
			}
			else if (FMemory::Memcmp(InBoxInfo.UUID, UUID_tenc, 16) == 0)
			{
				MP4Utilities::FMP4BoxInfo WellKnown(InBoxInfo);
				WellKnown.Type = MP4Utilities::MakeBoxAtom('t','e','n','c');
				return FMP4BoxTENC::Create(InParent, WellKnown);
			}
			else if (FMemory::Memcmp(InBoxInfo.UUID, UUID_senc, 16) == 0)
			{
				MP4Utilities::FMP4BoxInfo WellKnown(InBoxInfo);
				WellKnown.Type = MP4Utilities::MakeBoxAtom('s','e','n','c');
				return FMP4BoxSENC::Create(InParent, WellKnown);
			}
			else
			{
				// Unknown UUID, generate a base box.
				return FMP4BoxBase::Create(InParent, InBoxInfo);
			}
		}

		//UE_LOGF(LogTemp, Log, "unhandled box \"%ls\"", *FString::ConstructFromPtrSize((const ANSICHAR*)InBoxInfo.Name, 4));

		return FMP4BoxBase::Create(InParent, InBoxInfo);
	}


} // namespace MP4Boxes
