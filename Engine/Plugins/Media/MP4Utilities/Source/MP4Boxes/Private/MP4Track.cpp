// Copyright Epic Games, Inc. All Rights Reserved.

#include "MP4Track.h"
#include "MP4BoxesModule.h"

namespace MP4Boxes
{

	bool FMP4Track::FFragmentInfo::Prepare(const TSharedPtr<FMP4BoxTRAK, ESPMode::ThreadSafe>& InTrakBox, const TSharedPtr<FMP4BoxMVEX, ESPMode::ThreadSafe>& InMvexBox, const TArray<TSharedPtr<FMP4BoxBase>>& InAllRootBoxes)
	{
		if (bHasBeenPrepared)
		{
			return true;
		}
		if (!InTrakBox || !InMvexBox || InAllRootBoxes.IsEmpty())
		{
			return false;
		}
		TkhdBox = InTrakBox->FindBoxRecursive<FMP4BoxTKHD>(MP4Utilities::MakeBoxAtom('t','k','h','d'), 0);
		MdhdBox = InTrakBox->FindBoxRecursive<FMP4BoxMDHD>(MP4Utilities::MakeBoxAtom('m','d','h','d'), 1);
		TencBox = InTrakBox->FindBoxRecursive<FMP4BoxTENC>(MP4Utilities::MakeBoxAtom('t','e','n','c'), 10);

		auto _StszBox = InTrakBox->FindBoxRecursive<FMP4BoxSTSZ>(MP4Utilities::MakeBoxAtom('s','t','s','z'), 5);

		check(TkhdBox && MdhdBox && _StszBox);
		if (!TkhdBox || !MdhdBox || !_StszBox)
		{
			return false;
		}
		// Get the ID of this track.
		const uint32 TrackID = TkhdBox->GetTrackID();
		// Locate the mandatory `trex` box.
		TrexBox = InMvexBox->FindBoxRecursiveByPredicate<FMP4BoxTREX>(MP4Utilities::MakeBoxAtom('t','r','e','x'),[tkid=TrackID](const TSharedPtr<FMP4BoxTREX, ESPMode::ThreadSafe>& InTrex)
				{ return InTrex->GetTrackID() == tkid; }, 0);
		if (!TrexBox)
		{
			return false;
		}
		// Locate all `moof` boxes containing track data for this track and collect their `traf` boxes.
		TArray<TSharedPtr<FMP4BoxTRAF, ESPMode::ThreadSafe>> TrackTrafBoxes;
		for(const TSharedPtr<FMP4BoxBase>& rb : InAllRootBoxes)
		{
			if (rb->GetType() != MP4Utilities::MakeBoxAtom('m','o','o','f'))
			{
				continue;
			}
			rb->GetAllBoxInstancesByPredicate(TrackTrafBoxes, MP4Utilities::MakeBoxAtom('t','r','a','f'), [tkid=TrackID](const TSharedPtr<FMP4BoxTRAF, ESPMode::ThreadSafe>& InTraf)
			{
				auto Tfhd = InTraf->FindBoxRecursive<FMP4BoxTFHD>(MP4Utilities::MakeBoxAtom('t','f','h','d'), 0);
				return Tfhd && Tfhd->GetTrackID() == tkid;
			});
		}

		// How many samples are in the `moov`?
		NumBaseTrackSamples = _StszBox->GetNumberOfSamples();

		// Set up a list of the matching `traf` boxes and their related info boxes for convenience.
		if (TrackTrafBoxes.Num())
		{
			NumTotalFragmentSamples = 0;
			TrackFragments.SetNum(TrackTrafBoxes.Num());
			for(int32 i=0, iMax=TrackTrafBoxes.Num(); i<iMax; ++i)
			{
				auto Frg = MakeShared<FTrackTrafs, ESPMode::ThreadSafe>();
				Frg->TrafBox = TrackTrafBoxes[i];
				Frg->MoofBox = Frg->TrafBox->FindParentBox<FMP4BoxMOOF>(MP4Utilities::MakeBoxAtom('m','o','o','f'));
				Frg->TfhdBox = Frg->TrafBox->FindBoxRecursive<FMP4BoxTFHD>(MP4Utilities::MakeBoxAtom('t','f','h','d'), 0);
				Frg->SencBox = Frg->TrafBox->FindBoxRecursive<FMP4BoxSENC>(MP4Utilities::MakeBoxAtom('s','e','n','c'), 0);
				if (Frg->SencBox)
				{
					Frg->SencBox->Prepare(TencBox);
				}
				Frg->TrafBox->GetAllBoxInstances<FMP4BoxTRUN>(Frg->TrunBoxes, MP4Utilities::MakeBoxAtom('t','r','u','n'));
				Frg->TrafBox->GetAllBoxInstances<FMP4BoxSUBS>(Frg->SubsBoxes, MP4Utilities::MakeBoxAtom('s','u','b','s'));
				Frg->AbsoluteFirstSampleNumber = NumBaseTrackSamples + NumTotalFragmentSamples;
				auto TfdtBox = Frg->TrafBox->FindBoxRecursive<FMP4BoxTFDT>(MP4Utilities::MakeBoxAtom('t','f','d','t'), 0);
				if (TfdtBox)
				{
					Frg->BaseMediaDecodeTime = TfdtBox->GetBaseMediaDecodeTime();
				}

				Frg->NumSamples = 0;
				for(int32 j=0, jMax=Frg->TrunBoxes.Num(); j<jMax; ++j)
				{
					Frg->NumSamples += Frg->TrunBoxes[j]->GetNumberOfSamples();
				}
				NumTotalFragmentSamples += Frg->NumSamples;
				TrackFragments[i] = MoveTemp(Frg);
			}
		}
		bHasBeenPrepared = true;
		return true;
	}

	void FMP4Track::FFragmentInfo::CalculateDurations()
	{
		if (!bDidCalculateDuration)
		{
			bDidCalculateDuration = true;

			BaseTrackDuration = 0;
			auto Mdia = MdhdBox->GetParentBox();
			check(Mdia.IsValid());
			if (Mdia.IsValid())
			{
				auto Stts = Mdia->FindBoxRecursive<FMP4BoxSTTS>(MP4Utilities::MakeBoxAtom('s','t','t','s'), 5);
				check(Stts.IsValid());
				if (Stts.IsValid())
				{
					BaseTrackDuration = Stts->GetTotalDuration();
				}
			}

			SumOfFragmentDurations = 0;
			for(const TSharedPtr<FTrackTrafs, ESPMode::ThreadSafe>& tf : TrackFragments)
			{
				tf->Duration = 0;
				for(const TSharedPtr<FMP4BoxTRUN, ESPMode::ThreadSafe>& tr : tf->TrunBoxes)
				{
					if (tr->HasSampleDurations())
					{
						tf->Duration += tr->GetTotalDuration();
					}
					else
					{
						if (tf->TfhdBox->HasDefaultSampleDuration())
						{
							tf->Duration += tr->GetNumberOfSamples() * tf->TfhdBox->GetDefaultSampleDuration();
						}
						else
						{
							tf->Duration += tr->GetNumberOfSamples() * TrexBox->GetDefaultSampleDuration();
						}
					}
				}
				if (tf->BaseMediaDecodeTime.IsSet())
				{
					//int64 ExpectedDTS = BaseTrackDuration + SumOfFragmentDurations;
					int64 bmdt = (int64)tf->BaseMediaDecodeTime.GetValue();
					tf->FirstTrackTotalDTS = bmdt;
				}
				else
				{
					tf->FirstTrackTotalDTS = BaseTrackDuration + SumOfFragmentDurations;
				}
				tf->LastTrackTotalDTSAndDur = tf->FirstTrackTotalDTS + tf->Duration;
				SumOfFragmentDurations += tf->Duration;
			}
		}
	}

	int64 FMP4Track::FFragmentInfo::GetTotalDuration()
	{
		CalculateDurations();
		return BaseTrackDuration + SumOfFragmentDurations;
	}










	bool FMP4Track::Prepare(MP4Utilities::FFractionalTime InFullMovieDuration, MP4Utilities::FFractionalTime InAdjustedMovieDuration)
	{
		if (!TrakBox.IsValid())
		{
			LastErrorMessage = FString::Printf(TEXT("No `trak` box given."));
			return false;
		}
		TkhdBox = TrakBox->FindBoxRecursive<FMP4BoxTKHD>(MP4Utilities::MakeBoxAtom('t','k','h','d'), 0);
		if (!TkhdBox.IsValid())
		{
			LastErrorMessage = FString::Printf(TEXT("No `tkhd` box in `trak`."));
			return false;
		}
		Convs.TrackID = TkhdBox->GetTrackID();
		Convs.FullMovieDuration = InFullMovieDuration;
		Convs.DurationFromMvhdBox = InAdjustedMovieDuration;

		// Check for correct box hierarchy.
		TSharedPtr<FMP4BoxMDIA, ESPMode::ThreadSafe> MdiaBox = TrakBox->FindBoxRecursive<FMP4BoxMDIA>(MP4Utilities::MakeBoxAtom('m','d','i','a'), 0);
		if (!MdiaBox.IsValid())
		{
			LastErrorMessage = FString::Printf(TEXT("No `mdia` box in `trak`."));
			return false;
		}
		TSharedPtr<FMP4BoxMINF, ESPMode::ThreadSafe> MinfBox = MdiaBox->FindBoxRecursive<FMP4BoxMINF>(MP4Utilities::MakeBoxAtom('m','i','n','f'), 0);
		if (!MinfBox.IsValid())
		{
			LastErrorMessage = FString::Printf(TEXT("No `minf` box in `mdia`."));
			return false;
		}
		TSharedPtr<FMP4BoxSTBL, ESPMode::ThreadSafe> StblBox = MinfBox->FindBoxRecursive<FMP4BoxSTBL>(MP4Utilities::MakeBoxAtom('s','t','b','l'), 0);
		if (!StblBox.IsValid())
		{
			LastErrorMessage = FString::Printf(TEXT("No `stbl` box in `minf`."));
			return false;
		}

		MdhdBox = MdiaBox->FindBoxRecursive<FMP4BoxMDHD>(MP4Utilities::MakeBoxAtom('m','d','h','d'), 0);
		if (!MdhdBox.IsValid())
		{
			LastErrorMessage = FString::Printf(TEXT("No `mdhd` box in `mdia`."));
			return false;
		}
		Convs.DurationFromMdhdBox = MdhdBox->GetDuration();
		if (Convs.DurationFromMdhdBox.GetDenominator() == 0)
		{
			LastErrorMessage = FString::Printf(TEXT("Timescale in `mdhd` box is zero, which is not supported."));
			return false;
		}
		// If this is a fragmented file then the duration in the mdhd box does not necessarily describe the entire media.
		// Get it from the calculated duration of all fragments.
		if (FragmentInfo.IsValid())
		{
			check(FragmentInfo->HasBeenPrepared());
			if (!FragmentInfo->HasBeenPrepared())
			{
				LastErrorMessage = FString::Printf(TEXT("Fragment information has not been set up."));
				return false;
			}
			Convs.DurationFromTkhdBox.SetFromND(FragmentInfo->GetTotalDuration(), Convs.DurationFromMdhdBox.GetDenominator());
			Convs.DurationFromMdhdBox.SetNumerator(FragmentInfo->GetTotalDuration());
		}
		else
		{
			Convs.DurationFromTkhdBox.SetFromND(TkhdBox->GetDuration(), Convs.DurationFromMvhdBox.GetDenominator());
		}

		// Required sample information boxes:
		SttsBox = StblBox->FindBoxRecursive<FMP4BoxSTTS>(MP4Utilities::MakeBoxAtom('s','t','t','s'), 0);
		if (!SttsBox.IsValid())
		{
			LastErrorMessage = FString::Printf(TEXT("No `stts` box in `stbl`."));
			return false;
		}
		StscBox = StblBox->FindBoxRecursive<FMP4BoxSTSC>(MP4Utilities::MakeBoxAtom('s','t','s','c'), 0);
		if (!StscBox.IsValid())
		{
			LastErrorMessage = FString::Printf(TEXT("No `stsc` box in `stbl`."));
			return false;
		}
		StszBox = StblBox->FindBoxRecursive<FMP4BoxSTSZ>(MP4Utilities::MakeBoxAtom('s','t','s','z'), 0);
		if (!StszBox.IsValid())
		{
			LastErrorMessage = FString::Printf(TEXT("No `stsz` box in `stbl`."));
			return false;
		}
		StcoBox = StblBox->FindBoxRecursive<FMP4BoxSTCO>(MP4Utilities::MakeBoxAtom('s','t','c','o'), 0);
		if (!StcoBox.IsValid())
		{
			LastErrorMessage = FString::Printf(TEXT("No `stco` ot `co64` box in `stbl`."));
			return false;
		}

		// Validity check
		Convs.NumMoovSamples = StszBox->GetNumberOfSamples();
		if (Convs.NumMoovSamples != SttsBox->GetNumTotalSamples())
		{
			LastErrorMessage = FString::Printf(TEXT("Mismatching number of samples in `stts` and `stsz` boxes."));
			return false;
		}
		Convs.NumTotalSamples = Convs.NumMoovSamples + (FragmentInfo.IsValid() ? FragmentInfo->NumTotalFragmentSamples : 0);

		// Optional sample information boxes:
		CttsBox = StblBox->FindBoxRecursive<FMP4BoxCTTS>(MP4Utilities::MakeBoxAtom('c','t','t','s'), 0);
		StssBox = StblBox->FindBoxRecursive<FMP4BoxSTSS>(MP4Utilities::MakeBoxAtom('s','t','s','s'), 0);
		TencBox = StblBox->FindBoxRecursive<FMP4BoxTENC>(MP4Utilities::MakeBoxAtom('t','e','n','c'), 6);
		SencBox = TrakBox->FindBoxRecursive<FMP4BoxSENC>(MP4Utilities::MakeBoxAtom('s','e','n','c'), 1);
		if (TencBox && SencBox)
		{
			SencBox->Prepare(TencBox);
		}
		StblBox->GetAllBoxInstances(SubsBoxes, MP4Utilities::MakeBoxAtom('s','u','b','s'));
		StblBox->GetAllBoxInstances(SgpdBoxes, MP4Utilities::MakeBoxAtom('s','g','p','d'));
		StblBox->GetAllBoxInstances(SbgpBoxes, MP4Utilities::MakeBoxAtom('s','b','g','p'));

		// Start with default values for what is mapped onto the timeline.
		Convs.CompositionTimeAtZeroPoint = 0;
		if (CttsBox.IsValid() && CttsBox->GetEntries().Num())
		{
			Convs.CompositionTimeAtZeroPoint = CttsBox->GetEntries()[0].sample_offset;
		}
		else if (FragmentInfo && FragmentInfo->TrackFragments.Num() && FragmentInfo->TrackFragments[0]->TrunBoxes.Num())
		{
			auto Trun = FragmentInfo->TrackFragments[0]->TrunBoxes[0];
			if (Trun->GetNumberOfSamples() && Trun->HasSampleCompositionTimeOffsets())
			{
				Convs.CompositionTimeAtZeroPoint = Trun->GetSampleCompositionTimeOffsets()[0];
			}
		}

		Convs.MappedDurationFromElstBox = Convs.DurationFromMvhdBox;
		// Optional edit list
		ElstBox = TrakBox->FindBoxRecursive<FMP4BoxELST>(MP4Utilities::MakeBoxAtom('e','l','s','t'), 1);
		// If there is an edit list it needs to be simple and only contain a composition time mapping.
		if (ElstBox.IsValid())
		{
			if (ElstBox->RepeatEdits())
			{
				LastErrorMessage = FString::Printf(TEXT("Repeating `elst` box ist not supported."));
				return false;
			}
			// If there is more than a single entry things are bound to be complicated, so we don't even
			// want to know if the entry we are interested in is there.
			const TArray<FMP4BoxELST::FEntry>& ElstEntries = ElstBox->GetEntries();
			if (ElstEntries.Num() == 0)
			{
				LastErrorMessage = FString::Printf(TEXT("Edit list is empty."));
				return false;
			}
			// Look for the first entry that maps media time to establish the start offset.
			// Any other entries we ignore.
			int32 Idx=-1;
			for(int32 i=0; i<ElstEntries.Num(); ++i)
			{
				if (ElstEntries[i].media_time >= 0)
				{
					Idx = i;
					break;
				}
			}
			if (Idx != -1)
			{
				// The `media_time` in the edit list entry is specified in composition time. Typically an entry is used to shift
				// a track having non-zero composition times to zero. In that case the `media_time` should correspond to the
				// first composition time offset, or be greater.
				// If it is less then the entry maps non-existing media, meaning that it essentially inserts an empty edit,
				// which we do not support (what should be displayed then?)
				if (ElstEntries[Idx].media_time < Convs.CompositionTimeAtZeroPoint)
				{
					// We assume that this is not actually wanted but a problem with the tool creating the file,
					// so we emit a warning and ignore the edits `media_time`
					UE_LOGF(LogMP4Boxes, Verbose, "Edit list entry of track #%u maps non-existent media at composition time %lld to the timeline track. First available media composition time is %lld. Empty media will be ignored.", Convs.TrackID, (long long int)ElstEntries[0].media_time, (long long int)Convs.CompositionTimeAtZeroPoint);
				}
				else
				{
					Convs.CompositionTimeAtZeroPoint = ElstEntries[Idx].media_time;
				}

				// The mapped duration may be different from the media duration itself, in which case the track is either truncated
				// or the last sample has a longer duration to be repeated until the end.
				if (ElstEntries[Idx].edit_duration == 0)
				{
					// The value of 0 is reserved for fragmented files with no `mehd` box.
					if (!FragmentInfo.IsValid())
					{
						LastErrorMessage = FString::Printf(TEXT("Edit list specifies zero edit duration, which is not supported."));
						return false;
					}
					// Otherwise the mapped duration is already set as the entire movie duration.
				}
				else
				{
					Convs.MappedDurationFromElstBox.SetFromND(ElstEntries[Idx].edit_duration, Convs.DurationFromMvhdBox.GetDenominator());
				}
			}
		}
		// For convenience sake convert the mapped duration from `mvhd` timescale into `mdhd` timescale.
		Convs.MappedDurationFromElstBox.SetFromND(Convs.MappedDurationFromElstBox.GetAsTimebase(Convs.DurationFromMdhdBox.GetDenominator()), Convs.DurationFromMdhdBox.GetDenominator());


		// Set up common track meta data
		UdtaBox = TrakBox->FindBoxRecursive<FMP4BoxUDTA>(MP4Utilities::MakeBoxAtom('u','d','t','a'), 0);
		if (UdtaBox.IsValid())
		{
			// Is there a `name` box?
			TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> NameBox = UdtaBox->FindBoxRecursive<FMP4BoxBase>(MP4Utilities::MakeBoxAtom('n','a','m','e'), 0);
			if (NameBox.IsValid() && NameBox->GetBoxData().Num())
			{
				// Some tools write the `name` box as a simple box, others as a full box.
				// The `name` box is not defined by ISO/IEC 14496-12 but by Apple QuickTime (see https://developer.apple.com/documentation/quicktime-file-format/name_atom)
				// where it is defined as a full box, so technically all tools would have to write it as a full box.
				TConstArrayView<uint8> bd(NameBox->GetBoxData());
				// Check if the first byte, which would be the version when this is a full box, is small enough to actually be a version number
				// and also small enough to not be a printable character. Check the second byte as well, which would be bits 16-23 of the flags
				// which is assumed to be zero for a full `name` box.
				if (bd.Num() >= 4 && bd[0] < 10U && bd[1] == 0)
				{
					CommonMetadata.Name = FString::ConstructFromPtrSize(reinterpret_cast<const UTF8CHAR*>(NameBox->GetBoxData().GetData()+4), NameBox->GetBoxData().Num()-4);
				}
				else
				{
					CommonMetadata.Name = FString::ConstructFromPtrSize(reinterpret_cast<const UTF8CHAR*>(NameBox->GetBoxData().GetData()), NameBox->GetBoxData().Num());
				}
			}
		}
		TSharedPtr<FMP4BoxHDLR, ESPMode::ThreadSafe> HdlrBox = MdiaBox->FindBoxRecursive<FMP4BoxHDLR>(MP4Utilities::MakeBoxAtom('h','d','l','r'), 0);
		check(HdlrBox.IsValid())
		if (HdlrBox.IsValid())
		{
			CommonMetadata.HandlerName = HdlrBox->GetHandlerName();
		}

		// Get language from `mdhd` box first.
		CommonMetadata.LanguageCode = MdhdBox->GetLanguageCode639_2T();
		// Then, if there is an `elng` box that gives better information get it from there.
		TSharedPtr<FMP4BoxELNG, ESPMode::ThreadSafe> ElngBox = MdiaBox->FindBoxRecursive<FMP4BoxELNG>(MP4Utilities::MakeBoxAtom('e','l','n','g'), 0);
		if (ElngBox.IsValid())
		{
			CommonMetadata.LanguageTag = ElngBox->GetLanguageTag();
		}

		bHasBeenPrepared = true;

		// If we are called with no movie duration we assume this is for iterating a `moof` of a fragmented
		// file with a separate initialization segment. In this case we do not set up first and last sample
		// convenience information as it is not needed and is not guaranteed to find sync samples.
		if (!InFullMovieDuration.IsValid() && FragmentInfo.IsValid())
		{
			return true;
		}

		// Given the timeline mapping, locate the sample number that falls onto the start of the timeline
		// and last one falling onto the end of the timeline.
		TSharedPtr<FIterator, ESPMode::ThreadSafe> StartIt = CreateIterator();
		if (!StartIt.IsValid())
		{
			LastErrorMessage = FString::Printf(TEXT("Could not locate first media sample. Is this an empty mp4?"));
			return false;
		}
		const int64 pts0 = Convs.CompositionTimeAtZeroPoint;
		uint32 SyncSampleNum = 0;
		bool bFound = false;
		while(1)
		{
			if (StartIt->IsSyncOrRAPSample())
			{
				SyncSampleNum = StartIt->GetSampleNumber();
			}
			int64 s = StartIt->GetPTS().GetNumerator();
			int64 e = s + StartIt->GetDuration().GetNumerator();
			if (pts0 >= s && pts0 < e)
			{
				Convs.FirstSample.SampleNumber = StartIt->GetSampleNumber();
				Convs.FirstSample.SamplePTS = s;
				Convs.FirstSample.StartPTS = pts0;
				Convs.FirstSample.SyncSampleNumber = SyncSampleNum;
				Convs.DTSShiftAtZeroPoint = StartIt->GetDTS().GetNumerator();
				bFound = true;
				break;
			}
			if (!StartIt->Next())
			{
				break;
			}
		}
		check(bFound);

		TSharedPtr<FIterator, ESPMode::ThreadSafe> EndIt = CreateIteratorAtLastFrame();
		check(EndIt.IsValid());

		// Find the highest PTS
		uint32 HighestPTSIndex = ~0U;
		int64 HighestPTS = TNumericLimits<int64>::Min();
		int64 HighestEndPTS = 0;
		while(1)
		{
			int64 pts = EndIt->GetPTS().GetNumerator();
			if (pts > HighestPTS)
			{
				HighestPTS = pts;
				HighestEndPTS = pts + EndIt->GetDuration().GetNumerator();
				HighestPTSIndex = EndIt->GetSampleNumber();
			}
			if (EndIt->IsSyncOrRAPSample() || !EndIt->Prev())
			{
				break;
			}
		}
		check(HighestPTSIndex != ~0U);

		const int64 pts1 = Convs.CompositionTimeAtZeroPoint + Convs.MappedDurationFromElstBox.GetNumerator();
		// Is the mapped duration is greater or equal to what the media duration is?
		if (pts1 >= HighestEndPTS)
		{
			Convs.LastSample.SampleNumber = HighestPTSIndex;
			Convs.LastSample.LastSampleNumber = Convs.NumTotalSamples - 1;
			Convs.LastSample.SamplePTS = HighestPTS;
			Convs.LastSample.EndPTS = pts1;
			double PaddingDuration = (pts1 - HighestEndPTS) / (double)Convs.DurationFromMdhdBox.GetDenominator();
			if (PaddingDuration >= 0.001)
			{
				UE_LOGF(LogMP4Boxes, Verbose, "Last sample duration in track #%u will be extended by %#.5f seconds to align with the movie duration in the `mvhd` box.", Convs.TrackID, PaddingDuration);
			}
		}
		// The mapping truncates the media. Find where that is.
		else
		{
			EndIt = CreateIteratorAtLastFrame();
			bFound = false;
			// It is possible for the sample's display interval to fall into a non-existing
			// time range due to composition time offsets shifting a sample forward in time
			// (a predicted sample) without there being samples that will fall into the gap
			// (which would typically be bi-directionally predicted samples; either due to an encoder
			// not emitting these samples or a problem with the muxer).
			// If this happens it is best to track the last display interval ending on or before the
			// mapped timeline end and fall back to using that in this case.
			int64 LargestActiveSamplePTS = TNumericLimits<int64>::Min();
			TSharedPtr<FIterator, ESPMode::ThreadSafe> LargestActiveSamplePTSIt;
			auto AdjustLastSample = [&]() -> void
			{
				Convs.LastSample.EndPTS = pts1;
				Convs.LastSample.SampleNumber = EndIt->GetSampleNumber();
				// If it is a sync or rap sample then this is also the last sample we need to look at
				// as no later sample (in decode order) may be needed to decode this one.
				if (EndIt->IsSyncOrRAPSample() || EndIt->IsLast())
				{
					Convs.LastSample.LastSampleNumber = EndIt->GetSampleNumber();
				}
				else
				{
					// We need decode frames up to this PTS, meaning that everything that comes
					// earlier in decode order we need to decode as well.
					Convs.LastSample.LastSampleNumber = EndIt->GetSampleNumber();
					while(EndIt->Next())
					{
						int64 NextS = EndIt->GetDTS().GetNumerator();
						if (NextS > pts1)
						{
							break;
						}
						Convs.LastSample.LastSampleNumber = EndIt->GetSampleNumber();
					}
				}
			};
			while(!bFound)
			{
				int64 s = EndIt->GetPTS().GetNumerator();
				int64 e = s + EndIt->GetDuration().GetNumerator();
				if (pts1 > s && pts1 <= e)
				{
					// This sample contains the end of the mapped duration.
					Convs.LastSample.SamplePTS = s;
					AdjustLastSample();
					bFound = true;
				}
				else
				{
					// Remember which sample's display interval is closest to the mapped duration.
					if (e <= pts1 && s > LargestActiveSamplePTS)
					{
						LargestActiveSamplePTS = s;
						LargestActiveSamplePTSIt = EndIt->Clone();
					}
					// If we have a fallback position and are now on a sync sample then we can
					// leave the loop. Nothing earlier can be a better match.
					if (LargestActiveSamplePTSIt.IsValid() && EndIt->IsSyncOrRAPSample())
					{
						break;
					}
					if (!EndIt->Prev())
					{
						break;
					}
				}
			}

			if (!bFound && LargestActiveSamplePTSIt.IsValid())
			{
				EndIt = MoveTemp(LargestActiveSamplePTSIt);
				UE_LOGF(LogMP4Boxes, Verbose, "Mapped duration in track #%u (%lld) falls in a PTS gap; truncating at last visible sample (PTS=%lld).", Convs.TrackID, (long long int)pts1, (long long int)LargestActiveSamplePTS);
				Convs.LastSample.SamplePTS = LargestActiveSamplePTS;
				AdjustLastSample();
				bFound = true;
			}
			if (!bFound)
			{
				LastErrorMessage = FString::Printf(TEXT("Could not locate the last media sample mapped to the timeline."));
				return false;
			}
		}
		return true;
	}

	const FMP4TrackMetadataCommon& FMP4Track::GetCommonMetadata()
	{
		return CommonMetadata;
	}


	FString FMP4Track::GetLastError()
	{
		return LastErrorMessage;
	}


	// Returns the number of samples in this track.
	uint32 FMP4Track::GetNumberOfSamples()
	{
		check(bHasBeenPrepared);
		return bHasBeenPrepared ? Convs.NumTotalSamples : 0;
	}

	// Create an interator starting at the first sample.
	TSharedPtr<FMP4Track::FIterator, ESPMode::ThreadSafe> FMP4Track::CreateIterator()
	{
		check(bHasBeenPrepared);
		if (!bHasBeenPrepared)
		{
			LastErrorMessage = FString::Printf(TEXT("Track has not been prepared, cannot create an iterator."));
			return nullptr;
		}
		if (Convs.NumTotalSamples == 0)
		{
			LastErrorMessage = FString::Printf(TEXT("There are no samples in this track, cannot create an iterator."));
			return nullptr;
		}
		TUniquePtr<FIterator> It(new FIterator);
		It->Track = AsWeak();
		It->FragmentVars.TrafTrunIterator.SetFragmentInfo(FragmentInfo);
		// Copy convenience values into the iterator
		It->Convs = Convs;
		// Set up the box iterators.
		if (Convs.NumMoovSamples)
		{
			It->stszIt.SetBox(StszBox);
			It->sttsIt.SetBox(SttsBox);
			It->cttsIt.SetBox(CttsBox, Convs.NumMoovSamples);
			It->stcoIt.SetBox(StcoBox);
			It->stscIt.SetBox(StscBox, Convs.NumMoovSamples);
			It->stssIt.SetBox(StssBox, Convs.NumMoovSamples);
			// Are there subsample boxes? If yes, we use the first one only in the iterator for now.
			It->subsIt.SetBox(SubsBoxes.Num() ? SubsBoxes[0] : nullptr, Convs.NumMoovSamples);
			// Encryption?
			It->sencIt.SetBox(TencBox, SencBox, Convs.NumMoovSamples);
			// Do we have a `rap ` group?
			TSharedPtr<FMP4BoxSGPD, ESPMode::ThreadSafe>* rapSGPD = SgpdBoxes.FindByPredicate([](const TSharedPtr<FMP4BoxSGPD, ESPMode::ThreadSafe>& e){ return e->GetGroupingType() == MP4Utilities::MakeBoxAtom('r','a','p',' ');});
			TSharedPtr<FMP4BoxSBGP, ESPMode::ThreadSafe>* rapSBGP = SbgpBoxes.FindByPredicate([](const TSharedPtr<FMP4BoxSBGP, ESPMode::ThreadSafe>& e){ return e->GetGroupingType() == MP4Utilities::MakeBoxAtom('r','a','p',' ');});
			if (rapSGPD && rapSBGP)
			{
				It->rapIt.SetBox(*rapSBGP, (*rapSGPD)->GetDefaultGroupDescriptionIndex(), Convs.NumMoovSamples);
			}
			else
			{
				// Initialize the iterator such that it can be used to return "not a RAP" for every sample.
				It->rapIt.SetBox(nullptr, 0, Convs.NumMoovSamples);
			}
		}
		It->StartAt(0);
		It->Update();
		return MakeShareable(It.Release());
	}

	// Create an interator starting at the last sample (used when iterating in reverse, crossing back from the beginning to the end)
	TSharedPtr<FMP4Track::FIterator, ESPMode::ThreadSafe> FMP4Track::CreateIteratorAtLastFrame()
	{
		check(bHasBeenPrepared);
		if (!bHasBeenPrepared)
		{
			LastErrorMessage = FString::Printf(TEXT("Track has not been prepared, cannot create an iterator."));
			return nullptr;
		}
		if (Convs.NumTotalSamples == 0)
		{
			LastErrorMessage = FString::Printf(TEXT("There are no samples in this track, cannot create an iterator."));
			return nullptr;
		}
		return CreateIterator(Convs.NumTotalSamples - 1);
	}

	TSharedPtr<FMP4Track::FIterator, ESPMode::ThreadSafe> FMP4Track::CreateIterator(uint32 InAtSampleNumber)
	{
		TSharedPtr<FMP4Track::FIterator, ESPMode::ThreadSafe> It(CreateIterator());
		if (It.IsValid() && InAtSampleNumber)
		{
			// Clamp to last sample if passed number is too large.
			InAtSampleNumber = InAtSampleNumber < Convs.NumTotalSamples ? InAtSampleNumber : Convs.NumTotalSamples-1;
			It->StartAt(InAtSampleNumber);
			It->Update();
		}
		return It;
	}


	TSharedPtr<FMP4Track::FIterator, ESPMode::ThreadSafe> FMP4Track::CreateIteratorAtKeyframe(FTimespan InForTime, FTimespan InLaterTimeThreshold)
	{
		check(bHasBeenPrepared);
		if (!bHasBeenPrepared || !SttsBox.IsValid())
		{
			LastErrorMessage = FString::Printf(TEXT("Track has not been prepared, cannot create an iterator."));
			return nullptr;
		}
		const int64 TotalDuration = FragmentInfo.IsValid() ? FragmentInfo->GetTotalDuration() : SttsBox->GetTotalDuration();
		if (TotalDuration <=0 || !Convs.DurationFromMdhdBox.IsValid() || !Convs.MappedDurationFromElstBox.IsValid())
		{
			LastErrorMessage = FString::Printf(TEXT("Invalid track duration, cannot create an iterator."));
			return nullptr;
		}
		if (InForTime < FTimespan::Zero())
		{
			InForTime = FTimespan::Zero();
		}
		if (InLaterTimeThreshold < FTimespan::Zero())
		{
			InLaterTimeThreshold = FTimespan::Zero();
		}
		const uint32 TrackTimescale = Convs.DurationFromMdhdBox.GetDenominator();
		int64 LocalTrackTime = MP4Utilities::ConvertToTimescale(TrackTimescale, InForTime.GetTicks(), ETimespan::TicksPerSecond);
		// Clamp the time into the media time. The input may be larger than the media time, which is possible due to an
		// edit list mapping more content into the timeline than the media has. We need to find the frame in the media
		// though, so we clamp the time into the media time.
		LocalTrackTime = LocalTrackTime > Convs.DurationFromMdhdBox.GetNumerator() ? Convs.DurationFromMdhdBox.GetNumerator() : LocalTrackTime;
		int64 MaxLocalTrackTime = MP4Utilities::ConvertToTimescale(TrackTimescale, (InForTime + InLaterTimeThreshold).GetTicks(), ETimespan::TicksPerSecond);

		// Shift the search time into the media timeline.
		LocalTrackTime += Convs.CompositionTimeAtZeroPoint;
		MaxLocalTrackTime += Convs.CompositionTimeAtZeroPoint;

		int64 ApproxSampleNumber = Convs.NumTotalSamples ? LocalTrackTime * (Convs.NumTotalSamples-1) / TotalDuration : 0;
		check(ApproxSampleNumber <= 0xffffffff);


		TSharedPtr<FMP4Track::FIterator, ESPMode::ThreadSafe> ApproxFrameIt(CreateIterator((uint32)ApproxSampleNumber));
		if (!ApproxFrameIt.IsValid())
		{
			LastErrorMessage = FString::Printf(TEXT("Failed to create track iterator for sample #%lld with %u samples in track"), (long long int)ApproxSampleNumber, Convs.NumTotalSamples);
			return nullptr;
		}
		// Move the approximate iterator backwards or forwards towards the target time.
		// This should not be off by much unless variable frame rate is used with greatly varying durations
		// or an edit list cuts off significant amounts of the media.
		if (ApproxFrameIt->GetPTS().GetNumerator() > LocalTrackTime)
		{
			for(; !ApproxFrameIt->IsFirst() && ApproxFrameIt->GetPTS().GetNumerator() > LocalTrackTime; ApproxFrameIt->Prev())
			{ }
		}
		else if (ApproxFrameIt->GetPTS().GetNumerator()+ApproxFrameIt->GetDuration().GetNumerator() <= LocalTrackTime)
		{
			for(; !ApproxFrameIt->IsLast() && ApproxFrameIt->GetPTS().GetNumerator()+ApproxFrameIt->GetDuration().GetNumerator() <= LocalTrackTime; ApproxFrameIt->Next())
			{ }
		}
		// Locate the nearest earlier sync sample, which might be the current one already.
		TSharedPtr<FMP4Track::FIterator, ESPMode::ThreadSafe> SyncFrameIt(ApproxFrameIt->Clone());
		for(; !SyncFrameIt->IsFirst() && (SyncFrameIt->GetPTS().GetNumerator() > LocalTrackTime || !SyncFrameIt->IsSyncOrRAPSample()); SyncFrameIt->Prev())
		{ }
		TSharedPtr<FMP4Track::FIterator, ESPMode::ThreadSafe> NextSyncFrameIt(ApproxFrameIt->Clone());
		bool bLaterOneIsPossible = false;
		if (MaxLocalTrackTime > LocalTrackTime)
		{
			// Due to possible frame reordering we need to look at the DTS here with the composition offset applied
			// to be sure to find the correct sample. If we were to look at the PTS we could leave the loop to early.
			for(; !NextSyncFrameIt->IsLast() && (NextSyncFrameIt->GetDTS().GetNumerator() <= MaxLocalTrackTime && !NextSyncFrameIt->IsSyncOrRAPSample()); NextSyncFrameIt->Next())
			{ }
			bLaterOneIsPossible = NextSyncFrameIt->IsSyncOrRAPSample() && NextSyncFrameIt->GetPTS().GetNumerator() <= MaxLocalTrackTime;
		}
		// Did we even find any sync sample?
		if (!SyncFrameIt->IsSyncOrRAPSample() && !NextSyncFrameIt->IsSyncOrRAPSample())
		{
			LastErrorMessage = FString::Printf(TEXT("No sync sample found, cannot create an iterator."));
			return nullptr;
		}
		// If there is a possible later one to use we need to check if the earlier one is outside the threshold.
		if (bLaterOneIsPossible && LocalTrackTime - SyncFrameIt->GetPTS().GetNumerator() > MaxLocalTrackTime - LocalTrackTime)
		{
			return NextSyncFrameIt;
		}
		return SyncFrameIt;
	}

/****************************************************************************************************************************************************/
/****************************************************************************************************************************************************/
/****************************************************************************************************************************************************/

	void FMP4Track::FIterator::StartAt(uint32 InFrameNumber)
	{
		SampleNumber = InFrameNumber;
		if (InFrameNumber < Convs.NumMoovSamples)
		{
			stszIt.SetToSampleNumber(InFrameNumber);
			sttsIt.SetToSampleNumber(InFrameNumber);
			cttsIt.SetToSampleNumber(InFrameNumber);
			stcoIt.SetToSampleNumber(InFrameNumber);
			stscIt.SetToSampleNumber(InFrameNumber);
			stssIt.SetToSampleNumber(InFrameNumber);
			rapIt.SetToSampleNumber(InFrameNumber);
			subsIt.SetToSampleNumber(InFrameNumber);
			sencIt.SetToSampleNumber(InFrameNumber);
			FragmentVars.bInFragments = false;
		}
		else
		{
			FragmentVars.TrafTrunIterator.StartAt(InFrameNumber);
			FragmentVars.bInFragments = true;
		}
	}

	void FMP4Track::FIterator::Update()
	{
		if (!IsValid())
		{
			return;
		}
		// Are we in the file's base samples or the fragments (if any)?
		if (!FragmentVars.bInFragments)
		{
			// We are in the samples from the moov box.
			int64 DTS = sttsIt.GetCurrentTime();
			uint32 Duration = sttsIt.GetCurrentDuration();
			int64 CompositionTimeOffset = cttsIt.GetCurrentOffset();
			int64 PTS = DTS + CompositionTimeOffset;

			const uint32 Timescale = Convs.DurationFromMdhdBox.GetDenominator();
			CurrentDTS.SetFromND(DTS, Timescale);
			CurrentPTS.SetFromND(PTS, Timescale);
			CurrentEffectiveDTS.SetFromND(DTS - Convs.DTSShiftAtZeroPoint, Timescale);
			CurrentEffectivePTS.SetFromND(PTS - Convs.CompositionTimeAtZeroPoint, Timescale);
			// Set the duration as the fraction of the duration and the timescale.
			CurrentDuration.SetFromND(Duration, Timescale);
			// Also set the duration as the delta of the DTS of this sample and the next
			// in timespan units. This is to avoid transformation issues from media local
			// time into the timescale used in engine.
			CurrentDurationTS = MP4Utilities::FFractionalTime(DTS + Duration, Timescale).GetAsTimespan() - CurrentDTS.GetAsTimespan();
			CurrentBaseMediaDecodeTime.Reset();
			CurrentSampleSize = stszIt.GetCurrentSampleSize();
			bCurrentIsSyncOrRAP = stssIt.IsSyncSample() || rapIt.GetCurrentGroupDescriptionIndex() != 0;
			// Subsample list
			CurrentSubsampleList = subsIt.GetCurrentSubsamples();
			CurrentSubsampleBoxFlags = subsIt.GetSubsampleBoxFlags();
			// Encryption
			EncryptionEntry = sencIt.GetCurrentEntry();

			// Which chunk is this sample in?
			uint32 chunkIndex = stscIt.GetCurrentChunkIndex();
			check(chunkIndex);
			uint64 chunkOffset = stcoIt.GetOffsetForChunkIndex(chunkIndex - 1);
			check(chunkOffset);
			// Which sample position within the current chunk run are we at?
			uint32 samplePosInChunk = stscIt.GetSampleIndexInCurrentChunk();
			// Giving us which sample number at the start of the chunk?
			uint32 sampleNumAtChunkStart = SampleNumber - samplePosInChunk;
			for(uint32 i=0; i<samplePosInChunk; ++i)
			{
				chunkOffset += stszIt.GetSampleSizeForSampleNum(sampleNumAtChunkStart + i);
			}
			CurrentSampleFileOffset = (int64) chunkOffset;
		}
		else
		{
			// We are in the fragment samples.
			int64 DTS = FragmentVars.TrafTrunIterator.GetDTS();
			int64 PTS = FragmentVars.TrafTrunIterator.GetPTS();
			int64 Duration = FragmentVars.TrafTrunIterator.GetDuration();
			const uint32 Timescale = Convs.DurationFromMdhdBox.GetDenominator();
			CurrentDTS.SetFromND(DTS, Timescale);
			CurrentPTS.SetFromND(PTS, Timescale);
			CurrentEffectiveDTS.SetFromND(DTS - Convs.DTSShiftAtZeroPoint, Timescale);
			CurrentEffectivePTS.SetFromND(PTS - Convs.CompositionTimeAtZeroPoint, Timescale);
			// Set the duration as the fraction of the duration and the timescale.
			CurrentDuration.SetFromND(Duration, Timescale);
			// Also set the duration as the delta of the DTS of this sample and the next
			// in timespan units. This is to avoid transformation issues from media local
			// time into the timescale used in engine.
			CurrentDurationTS = MP4Utilities::FFractionalTime(DTS + Duration, Timescale).GetAsTimespan() - CurrentDTS.GetAsTimespan();
			if (FragmentVars.TrafTrunIterator.GetBaseMediaDecodeTime().IsSet())
			{
				CurrentBaseMediaDecodeTime = MP4Utilities::FFractionalTime((int64) FragmentVars.TrafTrunIterator.GetBaseMediaDecodeTime().GetValue(), Timescale);
			}
			else
			{
				CurrentBaseMediaDecodeTime.Reset();
			}
			CurrentSampleSize = FragmentVars.TrafTrunIterator.GetSampleSize();
			bCurrentIsSyncOrRAP = FragmentVars.TrafTrunIterator.IsSyncSample();
			CurrentSampleFileOffset = FragmentVars.TrafTrunIterator.GetAbsoluteOffset();
			SampleNumber = FragmentVars.TrafTrunIterator.GetAbsoluteSampleNumber();
			// Subsample list
			CurrentSubsampleList = FragmentVars.TrafTrunIterator.GetSubsampleList();
			CurrentSubsampleBoxFlags = FragmentVars.TrafTrunIterator.GetSubsampleBoxFlags();
			// Encryption
			EncryptionEntry = FragmentVars.TrafTrunIterator.GetEncryptionInfo();
		}
	}


	bool FMP4Track::FIterator::Next()
	{
		const uint32 NextSampleNumber = SampleNumber + 1;
		if (NextSampleNumber >= Convs.NumTotalSamples)
		{
			return false;
		}
		if (NextSampleNumber < Convs.NumMoovSamples)
		{
			// Note: stcoIt is not an iterator, so there's nothing to call on it.
			verify(stszIt.Next());
			verify(sttsIt.Next());
			verify(cttsIt.Next());
			verify(stscIt.Next());
			verify(stssIt.Next());
			verify(rapIt.Next());
			verify(subsIt.Next());
			verify(sencIt.Next());
		}
		else
		{
			// Already in fragments?
			if (FragmentVars.bInFragments)
			{
				if (!FragmentVars.TrafTrunIterator.Next())
				{
					check(!"why did we get here?");
					return false;
				}
			}
			else
			{
				StartAt(NextSampleNumber);
			}
		}
		SampleNumber = NextSampleNumber;
		Update();
		return true;
	}
	bool FMP4Track::FIterator::Prev()
	{
		if (SampleNumber == 0)
		{
			return false;
		}
		const uint32 NextSampleNumber = SampleNumber - 1;
		if (!FragmentVars.bInFragments)
		{
			// Note: stcoIt is not an iterator, so there's nothing to call on it.
			verify(stszIt.Prev());
			verify(sttsIt.Prev());
			verify(cttsIt.Prev());
			verify(stscIt.Prev());
			verify(stssIt.Prev());
			verify(rapIt.Prev());
			verify(subsIt.Prev());
			verify(sencIt.Prev());
		}
		else
		{
			// Going back from fragments into the moov?
			if (!FragmentVars.TrafTrunIterator.Prev())
			{
				StartAt(NextSampleNumber);
			}
		}
		SampleNumber = NextSampleNumber;
		Update();
		return true;
	}


	bool FMP4Track::FIterator::NextEffective()
	{
		// The last sample number is inclusive, that is, that sample is needed.
		if (SampleNumber+1 <= Convs.LastSample.LastSampleNumber)
		{
			return Next();
		}
		return false;
	}
	bool FMP4Track::FIterator::PrevEffective()
	{
		if (SampleNumber > Convs.FirstSample.SyncSampleNumber)
		{
			return Prev();
		}
		return false;
	}

	bool FMP4Track::FIterator::GetCurrentChunkRemainingSampleInfo(FChunkInfo& OutInfo) const
	{
		if (!IsValid())
		{
			return false;
		}
		if (!FragmentVars.bInFragments)
		{
			uint32 NumRem = stscIt.GetNumSamplesInCurrentChunk() - stscIt.GetSampleIndexInCurrentChunk();
			if (NumRem)
			{
				OutInfo.NumSamples = NumRem;
				auto STSZ = stszIt.GetBox();
				if (uint32 cs = STSZ->GetConstantSampleSize())
				{
					OutInfo.SizeInBytes = NumRem * cs;
				}
				else
				{
					OutInfo.SizeInBytes = 0;
					for(uint32 i=stszIt.GetCurrentSampleNum(),iMax=i+NumRem; i<iMax; ++i)
					{
						OutInfo.SizeInBytes += STSZ->GetSizeOfSample(i);
					}
				}
				if (int64 cd = sttsIt.GetConstantSampleDuration())
				{
					OutInfo.Duration = NumRem * cd;
				}
				else
				{
					OutInfo.Duration = 0;
					FSTTSBoxIterator durIt(sttsIt);
					for(uint32 i=0; i<NumRem; ++i)
					{
						OutInfo.Duration += durIt.GetCurrentDuration();
						durIt.Next();
					}
				}
			}
			else
			{
				OutInfo = {};
			}
			// This is the last chunk if there are no fragments following that we could move into.
			OutInfo.bIsLastChunk = stscIt.IsFinalChunk() && Convs.NumMoovSamples == Convs.NumTotalSamples;
			return true;
		}
		else
		{
			return FragmentVars.TrafTrunIterator.GetCurrentTrafRemainingSampleInfo(OutInfo);
		}
	}



	void FMP4Track::FIterator::FTrafTrunIterator::SetFragmentInfo(const TSharedPtr<FFragmentInfo, ESPMode::ThreadSafe>& InFragmentInfo)
	{
		FragmentInfo = InFragmentInfo;
		if (FragmentInfo && FragmentInfo->TrexBox)
		{
			TrexDefaultSampleDescriptionIndex = FragmentInfo->TrexBox->GetDefaultSampleDescriptionIndex();
			TrexDefaultSampleDuration = FragmentInfo->TrexBox->GetDefaultSampleDuration();
			TrexDefaultSampleSize = FragmentInfo->TrexBox->GetDefaultSampleSize();
			TrexDefaultSampleFlags = FragmentInfo->TrexBox->GetDefaultSampleFlags();
		}
		else
		{
			TrexDefaultSampleDescriptionIndex = 0;
			TrexDefaultSampleDuration = 0;
			TrexDefaultSampleSize = 0;
			TrexDefaultSampleFlags = 0;
		}
	}

	bool FMP4Track::FIterator::FTrafTrunIterator::StartAt(uint32 InSampleNum)
	{
		check(IsSetup());

		TrunNumber = 0;
		TSharedPtr<FFragmentInfo::FTrackTrafs, ESPMode::ThreadSafe> Traf;
		TSharedPtr<FMP4BoxTRUN, ESPMode::ThreadSafe> Trun;
		// In which fragment is the frame?
		for(int32 trfIdx=0,trfIdxMax=FragmentInfo->TrackFragments.Num(); !Trun && trfIdx<trfIdxMax; ++trfIdx)
		{
			Traf = FragmentInfo->TrackFragments[trfIdx];
			if (InSampleNum >= Traf->AbsoluteFirstSampleNumber &&
				InSampleNum < Traf->AbsoluteFirstSampleNumber + Traf->NumSamples)
			{
				FirstAbsoluteSampleNumber = Traf->AbsoluteFirstSampleNumber;
				TrunStartDTS = Traf->FirstTrackTotalDTS;
				uint32 FrameNumInTraf = InSampleNum - Traf->AbsoluteFirstSampleNumber;
				for(int32 i=0, iMax=Traf->TrunBoxes.Num(); i<iMax; ++i)
				{
					Trun = Traf->TrunBoxes[i];
					if (FrameNumInTraf < Traf->TrunBoxes[i]->GetNumberOfSamples())
					{
						TrafNumber = trfIdx;
						TrunNumber = i;
						SampleNumber = FrameNumInTraf;
						break;
					}
					if (Trun->HasSampleDurations())
					{
						TrunStartDTS += Trun->GetTotalDuration();
					}
					else
					{
						TrunStartDTS += Trun->GetNumberOfSamples() * (Traf->TfhdBox->HasDefaultSampleDuration() ? Traf->TfhdBox->GetDefaultSampleDuration() : TrexDefaultSampleDuration);
					}
					FirstAbsoluteSampleNumber += Trun->GetNumberOfSamples();
					FrameNumInTraf -= Trun->GetNumberOfSamples();
				}

				// While there can be multiple `trun` boxes there can only be one `subs` box since the
				// standard mandates that the `flags` field shall differ between multiple `subs` boxes,
				// but because the flags is codec specific and hence has a certain meaning it would
				// have to be the same in the `subs` boxes if there were several, which is forbidden.
				// See ISO/IEC 14496-12:2022 Section 8.7.7 Sub-sample information box

				// Are there subsample boxes? If yes, we use the first one only in the iterator for now.
				subsIt.SetBox(Traf->SubsBoxes.Num() ? Traf->SubsBoxes[0] : nullptr, Traf->NumSamples);
				subsIt.SetToSampleNumber(SampleNumber);

				// Likewise there can only be a single `senc` box.
				sencIt.SetBox(FragmentInfo->TencBox, Traf->SencBox, Traf->NumSamples);
				sencIt.SetToSampleNumber(SampleNumber);
			}
		}
		check(Trun.IsValid());
		if (!Trun)
		{
			return false;
		}

		Update(Traf);
		Update(Trun);
		return true;
	}

	void FMP4Track::FIterator::FTrafTrunIterator::Update(const TSharedPtr<FFragmentInfo::FTrackTrafs, ESPMode::ThreadSafe>& InTraf)
	{
		DefaultSampleDescriptionIndex = InTraf->TfhdBox->HasSampleDescriptionIndex() ? InTraf->TfhdBox->GetSampleDescriptionIndex() : TrexDefaultSampleDescriptionIndex;
		DefaultSampleDuration = InTraf->TfhdBox->HasDefaultSampleDuration() ? InTraf->TfhdBox->GetDefaultSampleDuration() : TrexDefaultSampleDuration;
		DefaultSampleSize = InTraf->TfhdBox->HasDefaultSampleSize() ? InTraf->TfhdBox->GetDefaultSampleSize() : TrexDefaultSampleSize;
		DefaultSampleFlags = InTraf->TfhdBox->HasDefaultSampleFlags() ? InTraf->TfhdBox->GetDefaultSampleFlags() : TrexDefaultSampleFlags;

		BaseOffset = 0;
		// Note: we do not consider any of the DataEntryImdaBox, DataEntrySeqNumImdaBox or IdentifiedMediaDataBox here.
		if (InTraf->TfhdBox->HasBaseDataOffset())
		{
			BaseOffset = InTraf->TfhdBox->GetBaseDataOffset();
		}
		else if (InTraf->TfhdBox->IsMoofDefaultBase())
		{
			BaseOffset = InTraf->MoofBox->GetBoxFileOffset();
		}
		else
		{
			BaseOffset = InTraf->MoofBox->GetBoxFileOffset();
		}

		BaseMediaDecodeTime = InTraf->BaseMediaDecodeTime;

		// Calculate the byte range of the first sample in the first `trun` up to and including the last sample in the last `trun`.
		// This does not necessarily mean that all samples are consecutive in the file as each `trun` has its own offset so `trun`'s
		// could as well be interleaved across sibling `traf`'s.
		// We only want to know the file byte range all the samples are to be found in.
		if (InTraf->TrunBoxes.Num())
		{
			TrunFirstSampleAbsoluteByteOffset = BaseOffset + (InTraf->TrunBoxes[0]->HasSampleOffset() ? InTraf->TrunBoxes[0]->GetSampleOffset() : 0);
			TrunLastSampleAbsoluteByteOffset = BaseOffset + (InTraf->TrunBoxes.Last()->HasSampleOffset() ? InTraf->TrunBoxes.Last()->GetSampleOffset() : 0);
			int64 LastSize = InTraf->TrunBoxes.Last()->HasSampleSizes() ? InTraf->TrunBoxes.Last()->GetTotalSampleSize() : InTraf->TrunBoxes.Last()->GetNumberOfSamples() * DefaultSampleSize;
			TrunLastSampleAbsoluteByteOffset += LastSize;
		}
	}

	void FMP4Track::FIterator::FTrafTrunIterator::Update(const TSharedPtr<FMP4BoxTRUN, ESPMode::ThreadSafe>& InTrun)
	{
		// Determine the offset to the sample by either adding the preceding sample sizes
		// or directly if fixed sample size.
		DataOffset = InTrun->HasSampleOffset() ? InTrun->GetSampleOffset() : 0;

		if (InTrun->HasSampleSizes())
		{
			const uint32* SampleSizes = InTrun->GetSampleSizes().GetData();
			SampleSize = SampleSizes[SampleNumber];
			for(int32 i=0; i<SampleNumber; ++i)
			{
				DataOffset += *SampleSizes++;
			}
		}
		else
		{
			SampleSize = DefaultSampleSize;
			DataOffset += SampleSize * SampleNumber;
		}
		AbsoluteSampleByteOffset = BaseOffset + DataOffset;

		// Sample flags
		SampleFlags = InTrun->HasSampleFlags() ? InTrun->GetSampleFlags()[SampleNumber] : (SampleNumber == 0 && InTrun->HasFirstSampleFlags()) ? InTrun->GetFirstSampleFlags() : DefaultSampleFlags;

		// Duration and DTS
		if (InTrun->HasSampleDurations())
		{
			DTS = 0;
			const uint32* SampleDurations = InTrun->GetSampleDurations().GetData();
			Duration = SampleDurations[SampleNumber];
			for(int32 i=0; i<SampleNumber; ++i)
			{
				DTS += *SampleDurations++;
			}
		}
		else
		{
			Duration = DefaultSampleDuration;
			DTS = Duration * SampleNumber;
		}
		DTS += TrunStartDTS;
		// PTS
		if (InTrun->HasSampleCompositionTimeOffsets())
		{
			PTS = DTS + InTrun->GetSampleCompositionTimeOffsets()[SampleNumber];
		}
		else
		{
			PTS = DTS;
		}
		// Subsample list
		SubsampleList = subsIt.GetCurrentSubsamples();
		// Encryption
		EncryptionEntry = sencIt.GetCurrentEntry();
	}

	bool FMP4Track::FIterator::FTrafTrunIterator::Prev()
	{
		check(IsSetup());
		// If we are at the start of the fragments there is no previous sample to go back to.
		if (TrafNumber <= 0 && TrunNumber <= 0 && SampleNumber <= 0)
		{
			return false;
		}

		// Step the subsample iterator. There is only one for all truns so we do not need to handle moving across multiple.
		subsIt.Prev();
		// Same for the encryption
		sencIt.Prev();

		TSharedPtr<FFragmentInfo::FTrackTrafs, ESPMode::ThreadSafe> Traf = FragmentInfo->TrackFragments[TrafNumber];
		TSharedPtr<FMP4BoxTRUN, ESPMode::ThreadSafe> Trun = Traf->TrunBoxes[TrunNumber];
		// Do we have a preceding sample in this trun?
		if (SampleNumber > 0)
		{
			--SampleNumber;
			SampleSize = Trun->HasSampleSizes() ? Trun->GetSampleSizes()[SampleNumber] : DefaultSampleSize;
			DataOffset -= SampleSize;
			AbsoluteSampleByteOffset -= SampleSize;
			SampleFlags = Trun->HasSampleFlags() ? Trun->GetSampleFlags()[SampleNumber] : (SampleNumber == 0 && Trun->HasFirstSampleFlags()) ? Trun->GetFirstSampleFlags() : DefaultSampleFlags;
			Duration = Trun->HasSampleDurations() ? Trun->GetSampleDurations()[SampleNumber] : DefaultSampleDuration;
			DTS = DTS - Duration;
			PTS = DTS + (Trun->HasSampleCompositionTimeOffsets() ? Trun->GetSampleCompositionTimeOffsets()[SampleNumber] : 0);
			// Update subsample list
			SubsampleList = subsIt.GetCurrentSubsamples();
			// Update encryption
			EncryptionEntry = sencIt.GetCurrentEntry();
			return true;
		}
		// Is there a trun before we can move into?
		if (TrunNumber > 0)
		{
			--TrunNumber;
			Trun = Traf->TrunBoxes[TrunNumber];
			// Adjust the trun base values down.
			const uint32 NumSamplesInTrun = Trun->GetNumberOfSamples();
			check(NumSamplesInTrun);
			FirstAbsoluteSampleNumber -= NumSamplesInTrun;
			int64 TrunDuration = Trun->HasSampleDurations() ? Trun->GetTotalDuration() : NumSamplesInTrun * DefaultSampleDuration;
			TrunStartDTS -= TrunDuration;
			SampleNumber = NumSamplesInTrun - 1;
			Update(Trun);
			return true;
		}
		// Is there an earlier traf to move into?
		if (TrafNumber > 0)
		{
			--TrafNumber;
			Traf = FragmentInfo->TrackFragments[TrafNumber];
			Update(Traf);
			FirstAbsoluteSampleNumber = Traf->AbsoluteFirstSampleNumber;
			TrunStartDTS = Traf->FirstTrackTotalDTS;
			// There should be trun boxes here.
			check(!Traf->TrunBoxes.IsEmpty());
			TrunNumber = Traf->TrunBoxes.Num() - 1;
			for(int32 i=0; i<TrunNumber; ++i)
			{
				Trun = Traf->TrunBoxes[i];
				FirstAbsoluteSampleNumber += Trun->GetNumberOfSamples();
				if (Trun->HasSampleDurations())
				{
					TrunStartDTS += Trun->GetTotalDuration();
				}
				else
				{
					TrunStartDTS += Trun->GetNumberOfSamples() * (Traf->TfhdBox->HasDefaultSampleDuration() ? Traf->TfhdBox->GetDefaultSampleDuration() : TrexDefaultSampleDuration);
				}
			}
			Trun = Traf->TrunBoxes[TrunNumber];
			// There should be samples in here.
			check(Trun->GetNumberOfSamples());
			SampleNumber = Trun->GetNumberOfSamples() - 1;

			// Are there subsample boxes? If yes, we use the first one only in the iterator for now.
			subsIt.SetBox(Traf->SubsBoxes.Num() ? Traf->SubsBoxes[0] : nullptr, Traf->NumSamples);
			subsIt.SetToSampleNumber(SampleNumber);

			// Encryption?
			sencIt.SetBox(FragmentInfo->TencBox, Traf->SencBox, Traf->NumSamples);
			sencIt.SetToSampleNumber(SampleNumber);

			Update(Trun);
			return true;
		}
		return false;
	}

	bool FMP4Track::FIterator::FTrafTrunIterator::Next()
	{
		check(IsSetup());
		check(TrafNumber >= 0 && TrunNumber >= 0 && SampleNumber >= 0);
		TSharedPtr<FFragmentInfo::FTrackTrafs, ESPMode::ThreadSafe> Traf = FragmentInfo->TrackFragments[TrafNumber];
		TSharedPtr<FMP4BoxTRUN, ESPMode::ThreadSafe> Trun = Traf->TrunBoxes[TrunNumber];

		// Advance the subsample iterator. There is only one for all truns so we do not need to handle moving across multiple.
		subsIt.Next();
		// Same for the encryption
		sencIt.Next();

		// Still samples in this trun?
		if (SampleNumber + 1 < (int32)Trun->GetNumberOfSamples())
		{
			++SampleNumber;
			DataOffset += SampleSize;
			AbsoluteSampleByteOffset += SampleSize;
			DTS += Duration;
			PTS = DTS + (Trun->HasSampleCompositionTimeOffsets() ? Trun->GetSampleCompositionTimeOffsets()[SampleNumber] : 0);
			SampleSize = Trun->HasSampleSizes() ? Trun->GetSampleSizes()[SampleNumber] : DefaultSampleSize;
			SampleFlags = Trun->HasSampleFlags() ? Trun->GetSampleFlags()[SampleNumber] : (SampleNumber == 0 && Trun->HasFirstSampleFlags()) ? Trun->GetFirstSampleFlags() : DefaultSampleFlags;
			Duration = Trun->HasSampleDurations() ? Trun->GetSampleDurations()[SampleNumber] : DefaultSampleDuration;
			// Update subsample list
			SubsampleList = subsIt.GetCurrentSubsamples();
			// Update encryption
			EncryptionEntry = sencIt.GetCurrentEntry();
			return true;
		}
		// Is there another trun following we can move into?
		if (TrunNumber + 1 < Traf->TrunBoxes.Num())
		{
			TrunStartDTS = DTS + Duration;
			FirstAbsoluteSampleNumber += Trun->GetNumberOfSamples();
			++TrunNumber;
			Trun = Traf->TrunBoxes[TrunNumber];
			SampleNumber = 0;
			Update(Trun);
			return true;
		}
		// Is there another traf we can move into?
		if (TrafNumber + 1 < FragmentInfo->TrackFragments.Num())
		{
			++TrafNumber;
			Traf = FragmentInfo->TrackFragments[TrafNumber];
			Update(Traf);
			FirstAbsoluteSampleNumber = Traf->AbsoluteFirstSampleNumber;
			TrunStartDTS = Traf->FirstTrackTotalDTS;
			// There should be trun boxes here.
			check(!Traf->TrunBoxes.IsEmpty());
			TrunNumber = 0;
			Trun = Traf->TrunBoxes[0];
			SampleNumber = 0;

			// Are there subsample boxes? If yes, we use the first one only in the iterator for now.
			subsIt.SetBox(Traf->SubsBoxes.Num() ? Traf->SubsBoxes[0] : nullptr, Traf->NumSamples);
			subsIt.SetToSampleNumber(SampleNumber);

			// Encryption?
			sencIt.SetBox(FragmentInfo->TencBox, Traf->SencBox, Traf->NumSamples);
			sencIt.SetToSampleNumber(SampleNumber);

			Update(Trun);
			return true;
		}
		return false;
	}

	bool FMP4Track::FIterator::FTrafTrunIterator::GetCurrentTrafRemainingSampleInfo(FChunkInfo& OutInfo) const
	{
		TSharedPtr<FFragmentInfo::FTrackTrafs, ESPMode::ThreadSafe> Traf = FragmentInfo->TrackFragments[TrafNumber];
		TSharedPtr<FMP4BoxTRUN, ESPMode::ThreadSafe> Trun = Traf->TrunBoxes[TrunNumber];
		const int64 LastTrafSampleNumber = Traf->AbsoluteFirstSampleNumber + Traf->NumSamples;
		const int64 ThisSampleNumber = FirstAbsoluteSampleNumber + SampleNumber;
		OutInfo.NumSamples = LastTrafSampleNumber - ThisSampleNumber;
		//OutInfo.SizeInBytes = SampleSize;
		OutInfo.SizeInBytes = TrunLastSampleAbsoluteByteOffset - GetAbsoluteOffset();
		OutInfo.Duration = Duration;
		OutInfo.bIsLastChunk = TrafNumber + 1 >= FragmentInfo->TrackFragments.Num();

		uint32 NextTrunSampleNumber = (uint32)SampleNumber + 1;
		int32 NextTrunNumber = TrunNumber + 1;
		bool bTrunHasSampleSizes = Trun->HasSampleSizes();
		bool bTrunHasSampleDurations = Trun->HasSampleDurations();
		while(1)
		{
			// Still samples in this trun?
			if (NextTrunSampleNumber < (int32)Trun->GetNumberOfSamples())
			{
				OutInfo.Duration += bTrunHasSampleDurations ? Trun->GetSampleDurations()[NextTrunSampleNumber] : DefaultSampleDuration;
				//OutInfo.SizeInBytes += bTrunHasSampleSizes ? Trun->GetSampleSizes()[NextTrunSampleNumber] : DefaultSampleSize;
				++NextTrunSampleNumber;
			}
			// Is there another trun following we can move into?
			else if (NextTrunNumber < Traf->TrunBoxes.Num())
			{
				Trun = Traf->TrunBoxes[NextTrunNumber];
				bTrunHasSampleSizes = Trun->HasSampleSizes();
				bTrunHasSampleDurations = Trun->HasSampleDurations();
				NextTrunSampleNumber = 0;
				++NextTrunNumber;
			}
			else
			{
				break;
			}
		}
		return true;
	}


} // namespace MP4Boxes
