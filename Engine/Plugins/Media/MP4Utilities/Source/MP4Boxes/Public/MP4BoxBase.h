// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Templates/PimplPtr.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "Misc/Timespan.h"

#include "MP4Utilities.h"

#define UE_API MP4BOXES_API

namespace MP4Boxes
{

	class FMP4BoxBase : public TSharedFromThis<FMP4BoxBase>
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxBase>(new FMP4BoxBase(InParent, InBoxInfo)); }
		virtual ~FMP4BoxBase() = default;

		UE_API virtual TArray<uint8> GetBoxDataRAW() const;
		virtual const MP4Utilities::FMP4BoxInfo& GetBoxInfo() const
		{ return BoxInfo; }
		virtual uint32 GetType() const
		{ return BoxInfo.Type; }
		virtual int64 GetBoxSize() const
		{ return BoxInfo.Size; }
		virtual TConstArrayView<uint8> GetBoxData() const
		{ return BoxInfo.Data; }
		virtual int64 GetBoxFileOffset() const
		{ return BoxInfo.Offset; }
		virtual int64 GetBoxDataOffset() const
		{ return BoxInfo.DataOffset; }
		virtual bool IsLeafBox() const
		{ return true; }
		virtual bool IsListOfEntries() const
		{ return false; }
		virtual bool IsSampleDescription() const
		{ return false; }

		//-------------------------------------------------------------------------
		// Parent box query methods
		//
		TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> GetParentBox() const
		{
			return ParentBox.Pin();
		}

		template<typename T>
		TSharedPtr<T, ESPMode::ThreadSafe> FindParentBox(uint32 InType) const
		{
			TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Parent = ParentBox.Pin();
			while(Parent.IsValid())
			{
				if (Parent->GetType() == InType)
				{
					return StaticCastSharedPtr<T>(Parent);
				}
				Parent = Parent->ParentBox.Pin();
			}
			return nullptr;
		}

		//-------------------------------------------------------------------------
		// Child box query methods
		//
		const TArray<TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe>> GetChildren() const
		{ return Children; }

		// Find the first box of the given type being a child of this box, or any child of a child box.
		template<typename T>
		TSharedPtr<T, ESPMode::ThreadSafe> FindBoxRecursive(uint32 InType, int32 InMaxDepth=8) const
		{
			// First pass, check all children if they are of the type
			for(int32 i=0,iMax=Children.Num(); i<iMax; ++i)
			{
				if (Children[i]->GetType() == InType)
				{
					return StaticCastSharedPtr<T>(Children[i]);
				}
			}
			// Second pass, check children recursively
			if (InMaxDepth > 0)
			{
				for(int32 i=0, iMax=Children.Num(); i<iMax; ++i)
				{
					TSharedPtr<T, ESPMode::ThreadSafe> Box = Children[i]->FindBoxRecursive<T>(InType, InMaxDepth - 1);
					if (Box.IsValid())
					{
						return Box;
					}
				}
			}
			return nullptr;
		}

		template<typename T, typename Predicate>
		TSharedPtr<T, ESPMode::ThreadSafe> FindBoxRecursiveByPredicate(uint32 InType, Predicate InPredicate, int32 InMaxDepth=8) const
		{
			// First pass, check all children if they are of the type
			for(int32 i=0,iMax=Children.Num(); i<iMax; ++i)
			{
				if (Children[i]->GetType() == InType)
				{
					TSharedPtr<T, ESPMode::ThreadSafe> Elem = StaticCastSharedPtr<T>(Children[i]);
					if (::Invoke(InPredicate, Elem))
					{
						return Elem;
					}
				}
			}
			// Second pass, check children recursively
			if (InMaxDepth > 0)
			{
				for(int32 i=0, iMax=Children.Num(); i<iMax; ++i)
				{
					TSharedPtr<T, ESPMode::ThreadSafe> Box = Children[i]->FindBoxRecursiveByPredicate<T, Predicate>(InType, InPredicate, InMaxDepth - 1);
					if (Box.IsValid())
					{
						return Box;
					}
				}
			}
			return nullptr;
		}


		// Returns all instances of a given box type from the direct children of THIS BOX ONLY.
		template<typename T>
		void GetAllBoxInstances(TArray<TSharedPtr<T, ESPMode::ThreadSafe>>& OutAllBoxes, uint32 InType) const
		{
			for(int32 i=0,iMax=Children.Num(); i<iMax; ++i)
			{
				if (Children[i]->GetType() == InType)
				{
					OutAllBoxes.Emplace(StaticCastSharedPtr<T>(Children[i]));
				}
			}
		}

		template<typename T, typename Predicate>
		void GetAllBoxInstancesByPredicate(TArray<TSharedPtr<T, ESPMode::ThreadSafe>>& OutAllBoxes, uint32 InType, Predicate InPredicate, int32 InMaxDepth=8) const
		{
			for(int32 i=0,iMax=Children.Num(); i<iMax; ++i)
			{
				if (Children[i]->GetType() == InType)
				{
					TSharedPtr<T, ESPMode::ThreadSafe> Elem = StaticCastSharedPtr<T>(Children[i]);
					if (::Invoke(InPredicate, Elem))
					{
						OutAllBoxes.Emplace(Elem);
					}
				}
			}
			if (InMaxDepth > 0)
			{
				for(int32 i=0, iMax=Children.Num(); i<iMax; ++i)
				{
					Children[i]->GetAllBoxInstancesByPredicate<T, Predicate>(OutAllBoxes, InType, InPredicate, InMaxDepth - 1);
				}
			}
		}



		// Called by building the box tree. Not to be called by user code.
		virtual bool AddChildBox(TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> InChildBox)
		{
			if (InChildBox.IsValid())
			{
				Children.Emplace(MoveTemp(InChildBox));
				return true;
			}
			return false;
		}

		UE_API virtual void ProcessBoxChildrenRecursively(MP4Utilities::FMP4AtomReader& InOutReader, const MP4Utilities::FMP4BoxInfo& InCurrentBoxInfo);

		void SetRootBoxData(const TSharedPtr<MP4Utilities::FMP4BoxInfo, ESPMode::ThreadSafe>& InRootBoxData)
		{ RootBoxData = InRootBoxData; }
	protected:
		FMP4BoxBase(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo) : ParentBox(InParent), BoxInfo(InBoxInfo)
		{ }

		TSharedPtr<MP4Utilities::FMP4BoxInfo, ESPMode::ThreadSafe> RootBoxData;
		TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe> ParentBox;
		TArray<TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe>> Children;
		MP4Utilities::FMP4BoxInfo BoxInfo;
	};



	class FMP4BoxTreeParser
	{
	public:
		UE_API bool ParseBoxTree(const TSharedPtr<MP4Utilities::FMP4BoxInfo, ESPMode::ThreadSafe>& InRootBox);
		TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> GetBoxTree() const
		{ return BoxTree; }
	private:
		UE_API bool ParseBoxTreeInternal(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBox);
		TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> BoxTree;
	};

} // namespace MP4Boxes

#undef UE_API
