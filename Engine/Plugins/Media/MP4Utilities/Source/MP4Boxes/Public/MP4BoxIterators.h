// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "Misc/Timespan.h"

#include "MP4Boxes.h"

namespace MP4Boxes
{

	/****************************************************************************************************************************************************/

	struct FSTTSBoxIterator
	{
		void SetBox(const TSharedPtr<FMP4BoxSTTS, ESPMode::ThreadSafe>& InSttsBox)
		{
			SttsBox = InSttsBox;
			NumTotalSamples = SttsBox->GetNumTotalSamples();
			ConstantSampleDuration = SttsBox->GetConstantSampleDuration();
			Time = 0;
			CurrentSampleNum = 0;
			LocalPosInCurrent = 0;
			CurrentDuration = NumTotalSamples ? SttsBox->GetEntries()[0].sample_delta : 0;
			CurrentEntryIndex = 0;
		}

		void SetToSampleNumber(uint32 InSampleNum)
		{
			check(SttsBox.IsValid());
			check(InSampleNum < NumTotalSamples);
			InSampleNum = InSampleNum < NumTotalSamples ? InSampleNum : NumTotalSamples - 1;
			const TArray<FMP4BoxSTTS::FEntry>& Entries(SttsBox->GetEntries());
			Time = 0;
			CurrentEntryIndex = 0;
			LocalPosInCurrent = 0;
			CurrentDuration = Entries[0].sample_delta;
			uint32 n = 0;
			// If the entire current entry is still before the wanted position we can skip over it.
			while(n + Entries[CurrentEntryIndex].sample_count <= InSampleNum)
			{
				n += Entries[CurrentEntryIndex].sample_count;
				Time += Entries[CurrentEntryIndex].sample_count * (int64)CurrentDuration;
				++CurrentEntryIndex;
				CurrentDuration = Entries[CurrentEntryIndex].sample_delta;
			}
			LocalPosInCurrent = InSampleNum - n;
			Time += LocalPosInCurrent * (int64)CurrentDuration;
			CurrentSampleNum = InSampleNum;
		}

		bool Next()
		{
			check(SttsBox.IsValid());
			if (IsLast())
			{
				return false;
			}
			Time += CurrentDuration;
			++CurrentSampleNum;
			// Does the current range still provide?
			const TArray<FMP4BoxSTTS::FEntry>& Entries(SttsBox->GetEntries());
			if (++LocalPosInCurrent >= Entries[CurrentEntryIndex].sample_count)
			{
				// No, continue with the next entry.
				++CurrentEntryIndex;
				CurrentDuration = Entries[CurrentEntryIndex].sample_delta;
				LocalPosInCurrent = 0;
			}
			return true;
		}

		bool Prev()
		{
			check(SttsBox.IsValid());
			if (IsFirst())
			{
				return false;
			}
			// Need to go back one entry?
			if (LocalPosInCurrent > 0)
			{
				// Not yet.
				--LocalPosInCurrent;
			}
			else
			{
				// Yes.
				const TArray<FMP4BoxSTTS::FEntry>& Entries(SttsBox->GetEntries());
				--CurrentEntryIndex;
				LocalPosInCurrent = Entries[CurrentEntryIndex].sample_count - 1;
				CurrentDuration = Entries[CurrentEntryIndex].sample_delta;
			}
			Time -= CurrentDuration;
			--CurrentSampleNum;
			return true;
		}

		int64 GetCurrentTime() const
		{ return Time; }
		int64 GetConstantSampleDuration() const
		{ return ConstantSampleDuration; }
		uint32 GetCurrentDuration() const
		{ return CurrentDuration; }
		uint32 GetCurrentSampleNum() const
		{ return CurrentSampleNum; }
		uint32 GetNumTotalSamples() const
		{ return NumTotalSamples; }
		bool IsFirst() const
		{ return CurrentSampleNum == 0; }
		bool IsLast() const
		{ return CurrentSampleNum + 1 >= NumTotalSamples; }

	private:
		TSharedPtr<FMP4BoxSTTS, ESPMode::ThreadSafe> SttsBox;
		int64 Time = 0;
		int64 ConstantSampleDuration = 0;
		uint32 CurrentSampleNum = 0;
		uint32 NumTotalSamples = 0;
		uint32 LocalPosInCurrent = 0;
		uint32 CurrentDuration = 0;
		int32 CurrentEntryIndex = 0;
	};

	/****************************************************************************************************************************************************/

	struct FCTTSBoxIterator
	{
		void SetBox(const TSharedPtr<FMP4BoxCTTS, ESPMode::ThreadSafe>& InCttsBox, uint32 InMaxSamples)
		{
			if (InCttsBox.IsValid())
			{
				CttsBox = InCttsBox;
				NumTotalSamples = CttsBox->GetNumTotalSamples();
				CurrentOffset = NumTotalSamples ? CttsBox->GetEntries()[0].sample_offset : 0;
			}
			else
			{
				NumTotalSamples = InMaxSamples;
				CurrentOffset = 0;
			}
			CurrentSampleNum = 0;
			LocalPosInCurrent = 0;
			CurrentEntryIndex = 0;
		}

		void SetToSampleNumber(uint32 InSampleNum)
		{
			check(InSampleNum < NumTotalSamples);
			InSampleNum = InSampleNum < NumTotalSamples ? InSampleNum : NumTotalSamples - 1;
			if (CttsBox.IsValid())
			{
				const TArray<FMP4BoxCTTS::FEntry>& Entries(CttsBox->GetEntries());
				CurrentEntryIndex = 0;
				LocalPosInCurrent = 0;
				CurrentOffset = Entries[0].sample_offset;
				uint32 n = 0;
				// If the entire current entry is still before the wanted position we can skip over it.
				while(n + Entries[CurrentEntryIndex].sample_count <= InSampleNum)
				{
					n += Entries[CurrentEntryIndex].sample_count;
					++CurrentEntryIndex;
					CurrentOffset = Entries[CurrentEntryIndex].sample_offset;
				}
				LocalPosInCurrent = InSampleNum - n;
			}
			CurrentSampleNum = InSampleNum;
		}

		bool Next()
		{
			if (IsLast())
			{
				return false;
			}
			if (CttsBox.IsValid())
			{
				// Does the current range still provide?
				const TArray<FMP4BoxCTTS::FEntry>& Entries(CttsBox->GetEntries());
				if (++LocalPosInCurrent >= Entries[CurrentEntryIndex].sample_count)
				{
					// No, continue with the next entry.
					++CurrentEntryIndex;
					CurrentOffset = Entries[CurrentEntryIndex].sample_offset;
					LocalPosInCurrent = 0;
				}
			}
			++CurrentSampleNum;
			return true;
		}

		bool Prev()
		{
			if (IsFirst())
			{
				return false;
			}
			if (CttsBox.IsValid())
			{
				// Need to go back one entry?
				if (LocalPosInCurrent > 0)
				{
					// Not yet.
					--LocalPosInCurrent;
				}
				else
				{
					// Yes.
					const TArray<FMP4BoxCTTS::FEntry>& Entries(CttsBox->GetEntries());
					--CurrentEntryIndex;
					LocalPosInCurrent = Entries[CurrentEntryIndex].sample_count - 1;
					CurrentOffset = Entries[CurrentEntryIndex].sample_offset;
				}
			}
			--CurrentSampleNum;
			return true;
		}

		int64 GetCurrentOffset() const
		{ return CurrentOffset; }
		uint32 GetCurrentSampleNum() const
		{ return CurrentSampleNum; }
		bool IsFirst() const
		{ return CurrentSampleNum == 0; }
		bool IsLast() const
		{ return CurrentSampleNum + 1 >= NumTotalSamples; }

	private:
		TSharedPtr<FMP4BoxCTTS, ESPMode::ThreadSafe> CttsBox;
		int64 CurrentOffset = 0;
		uint32 CurrentSampleNum = 0;
		uint32 NumTotalSamples = 0;
		uint32 LocalPosInCurrent = 0;
		int32 CurrentEntryIndex = 0;
	};

	/****************************************************************************************************************************************************/

	struct FSTSCBoxIterator
	{
		void SetBox(const TSharedPtr<FMP4BoxSTSC, ESPMode::ThreadSafe>& InStscBox, uint32 InMaxSamples)
		{
			StscBox = InStscBox;
			NumTotalSamples = InMaxSamples;
			CurrentSampleNum = 0;

			if (StscBox->GetEntries().Num())
			{
				const TArray<FMP4BoxSTSC::FEntry>& Entries(StscBox->GetEntries());
				Current_first_chunk = Entries[0].first_chunk;
				Current_samples_per_chunk = Entries[0].samples_per_chunk;
				Current_sample_description_index = Entries[0].sample_description_index;
			}
			else
			{
				Current_first_chunk = 0;
				Current_samples_per_chunk = 0;
			}
			CurrentEntryIndex = 0;
			LocalSampleInChunk = 0;
		}

		void SetToSampleNumber(uint32 InSampleNum)
		{
			if (StscBox.IsValid() && StscBox->GetEntries().Num())
			{
				InSampleNum = InSampleNum < NumTotalSamples ? InSampleNum : NumTotalSamples-1;
				const TArray<FMP4BoxSTSC::FEntry>& Entries(StscBox->GetEntries());
				const uint32 MaxEntries = Entries.Num();
				uint32 n=0;
				CurrentEntryIndex = 0;
				Current_first_chunk = Entries[0].first_chunk;
				Current_samples_per_chunk = Entries[0].samples_per_chunk;
				Current_sample_description_index = Entries[0].sample_description_index;
				while(1)
				{
					n += Current_samples_per_chunk;
					if (n > InSampleNum)
					{
						LocalSampleInChunk = Current_samples_per_chunk - (n - InSampleNum);
						break;
					}
					else
					{
						++Current_first_chunk;
						if (CurrentEntryIndex + 1 < MaxEntries && Current_first_chunk == Entries[CurrentEntryIndex + 1].first_chunk)
						{
							++CurrentEntryIndex;
							Current_first_chunk = Entries[CurrentEntryIndex].first_chunk;
							Current_samples_per_chunk = Entries[CurrentEntryIndex].samples_per_chunk;
							Current_sample_description_index = Entries[CurrentEntryIndex].sample_description_index;
						}
					}
				}
				CurrentSampleNum = InSampleNum;
			}
		}

		bool Next()
		{
			if (StscBox.IsValid())
			{
				if (IsLast())
				{
					return false;
				}

				++CurrentSampleNum;
				// Reached end of the current chunk run?
				if (++LocalSampleInChunk == Current_samples_per_chunk)
				{
					const TArray<FMP4BoxSTSC::FEntry>& Entries(StscBox->GetEntries());
					const uint32 MaxEntries = Entries.Num();
					LocalSampleInChunk = 0;
					++Current_first_chunk;
					if (CurrentEntryIndex + 1 < MaxEntries && Current_first_chunk == Entries[CurrentEntryIndex + 1].first_chunk)
					{
						++CurrentEntryIndex;
						Current_first_chunk = Entries[CurrentEntryIndex].first_chunk;
						Current_samples_per_chunk = Entries[CurrentEntryIndex].samples_per_chunk;
						Current_sample_description_index = Entries[CurrentEntryIndex].sample_description_index;
					}
				}
				return true;
			}
			return false;
		}

		bool Prev()
		{
			if (StscBox.IsValid())
			{
				if (IsFirst())
				{
					return false;
				}

				--CurrentSampleNum;
				// At the start of the current chunk run?
				if (LocalSampleInChunk > 0)
				{
					--LocalSampleInChunk;
				}
				else
				{
					const TArray<FMP4BoxSTSC::FEntry>& Entries(StscBox->GetEntries());
					check(Current_first_chunk);
					// Are we at the start of this chunk run?
					if (Current_first_chunk == Entries[CurrentEntryIndex].first_chunk)
					{
						// We need to go back an entry if there is one.
						if (CurrentEntryIndex)
						{
							--CurrentEntryIndex;
							Current_samples_per_chunk = Entries[CurrentEntryIndex].samples_per_chunk;
							Current_sample_description_index = Entries[CurrentEntryIndex].sample_description_index;
						}
					}
					--Current_first_chunk;
					LocalSampleInChunk = Current_samples_per_chunk - 1;
				}
				return true;
			}
			return false;
		}

		// Note: The chunk index is 1-based, so you need to subtract 1 to use as an index into the `stco` box.
		uint32 GetCurrentChunkIndex() const
		{ return Current_first_chunk; }
		uint32 GetNumSamplesInCurrentChunk() const
		{ return Current_samples_per_chunk; }
		uint32 GetSampleIndexInCurrentChunk() const
		{ return LocalSampleInChunk; }
		uint32 GetCurrentSampleNum() const
		{ return CurrentSampleNum; }
		bool IsFirst() const
		{ return CurrentSampleNum == 0; }
		bool IsLast() const
		{ return CurrentSampleNum + 1 >= NumTotalSamples; }

		bool IsFinalChunk() const
		{
			if (!StscBox.IsValid())
			{
				return true;
			}
			const uint32 NumSmpInLast = StscBox->GetEntries().Num() ? StscBox->GetEntries().Last().samples_per_chunk : 0;
			const uint32 FirstSampleNumInLastChunk = NumTotalSamples - NumSmpInLast;
			return CurrentSampleNum >= FirstSampleNumInLastChunk;
		}
	private:
		TSharedPtr<FMP4BoxSTSC, ESPMode::ThreadSafe> StscBox;
		uint32 Current_first_chunk = 0;
		uint32 Current_samples_per_chunk = 0;
		uint32 Current_sample_description_index = 0;

		uint32 CurrentSampleNum = 0;
		uint32 NumTotalSamples = 0;
		uint32 LocalSampleInChunk = 0;
		uint32 CurrentEntryIndex = 0;
	};

	/****************************************************************************************************************************************************/

	struct FSTSZBoxIterator
	{
		void SetBox(const TSharedPtr<FMP4BoxSTSZ, ESPMode::ThreadSafe>& InStszBox)
		{
			check(InStszBox.IsValid());
			StszBox = InStszBox;
			NumTotalSamples = StszBox->GetNumberOfSamples();
			CurrentSampleNum = 0;
		}

		TSharedPtr<FMP4BoxSTSZ, ESPMode::ThreadSafe> GetBox() const
		{
			return StszBox;
		}

		void SetToSampleNumber(uint32 InSampleNum)
		{
			CurrentSampleNum = InSampleNum < NumTotalSamples ? InSampleNum : NumTotalSamples-1;
		}

		bool Next()
		{
			if (StszBox.IsValid())
			{
				if (IsLast())
				{
					return false;
				}
				++CurrentSampleNum;
				return true;
			}
			return false;
		}

		bool Prev()
		{
			if (StszBox.IsValid())
			{
				if (IsFirst())
				{
					return false;
				}
				--CurrentSampleNum;
				return true;
			}
			return false;
		}

		uint32 GetCurrentSampleSize() const
		{
			check(StszBox.IsValid());
			return StszBox->GetSizeOfSample(CurrentSampleNum);
		}

		uint32 GetSampleSizeForSampleNum(uint32 InForSampleNum) const
		{
			check(StszBox.IsValid());
			return StszBox->GetSizeOfSample(InForSampleNum);
		}

		uint32 GetNumTotalSamples() const
		{ return NumTotalSamples; }
		uint32 GetCurrentSampleNum() const
		{ return CurrentSampleNum; }
		bool IsFirst() const
		{ return CurrentSampleNum == 0; }
		bool IsLast() const
		{ return CurrentSampleNum + 1 >= NumTotalSamples; }

	private:
		TSharedPtr<FMP4BoxSTSZ, ESPMode::ThreadSafe> StszBox;
		uint32 NumTotalSamples = 0;
		uint32 CurrentSampleNum = 0;
	};

	/****************************************************************************************************************************************************/

	// This is not so much an iterator than a holder of the box.
	struct FSTCOBoxIterator
	{
		void SetBox(const TSharedPtr<FMP4BoxSTCO, ESPMode::ThreadSafe>& InStcoBox)
		{
			check(InStcoBox.IsValid());
			StcoBox = InStcoBox;
		}

		void SetToSampleNumber(uint32 InSampleNum)
		{
		}

		uint64 GetOffsetForChunkIndex(uint32 InChunkIndex) const
		{
			return StcoBox.IsValid() ? StcoBox->GetChunkOffset(InChunkIndex) : 0;
		}
	private:
		TSharedPtr<FMP4BoxSTCO, ESPMode::ThreadSafe> StcoBox;
	};

	/****************************************************************************************************************************************************/

	struct FSTSSBoxIterator
	{
		void SetBox(const TSharedPtr<FMP4BoxSTSS, ESPMode::ThreadSafe>& InStssBox, uint32 InMaxSamples)
		{
			StssBox = InStssBox;
			NumTotalSamples = InMaxSamples;
			CurrentSampleNum = 0;
			CurrentEntryIndex = 0;
			bCurrentIsSyncSample = !StssBox.IsValid() || (StssBox.IsValid() && StssBox->GetEntries().Num() && StssBox->GetEntries()[0] == 1);
		}

		void SetToSampleNumber(uint32 InSampleNum)
		{
			InSampleNum = InSampleNum < NumTotalSamples ? InSampleNum : NumTotalSamples-1;
			if (StssBox.IsValid())
			{
				const TArray<uint32>& Entries(StssBox->GetEntries());
				const uint32 SampleNumPlus1 = InSampleNum + 1;
				for(CurrentEntryIndex=0; CurrentEntryIndex<(uint32)Entries.Num(); ++CurrentEntryIndex)
				{
					if (Entries[CurrentEntryIndex] >= SampleNumPlus1)
					{
						break;
					}
				}
				bCurrentIsSyncSample = CurrentEntryIndex < (uint32)Entries.Num() ? Entries[CurrentEntryIndex] == SampleNumPlus1 : false;
			}
			CurrentSampleNum = InSampleNum;
		}

		bool Next()
		{
			if (IsLast())
			{
				return false;
			}
			++CurrentSampleNum;
			if (StssBox.IsValid())
			{
				const TArray<uint32>& Entries(StssBox->GetEntries());
				const uint32 SampleNumPlus1 = CurrentSampleNum + 1;
				while(CurrentEntryIndex < (uint32)Entries.Num() && Entries[CurrentEntryIndex] < SampleNumPlus1)
				{
					++CurrentEntryIndex;
				}
				bCurrentIsSyncSample = CurrentEntryIndex < (uint32)Entries.Num() ? Entries[CurrentEntryIndex] == SampleNumPlus1 : false;
			}
			return true;
		}

		bool Prev()
		{
			if (IsFirst())
			{
				return false;
			}
			--CurrentSampleNum;
			if (StssBox.IsValid())
			{
				const TArray<uint32>& Entries(StssBox->GetEntries());
				const uint32 SampleNumPlus1 = CurrentSampleNum + 1;
				if (Entries.Num())
				{
					CurrentEntryIndex = CurrentEntryIndex < (uint32)Entries.Num() ? CurrentEntryIndex : Entries.Num()-1;
					while(CurrentEntryIndex > 0 && Entries[CurrentEntryIndex] > SampleNumPlus1)
					{
						--CurrentEntryIndex;
					}
				}
				bCurrentIsSyncSample = CurrentEntryIndex < (uint32)Entries.Num() ? Entries[CurrentEntryIndex] == SampleNumPlus1 : false;
			}
			return true;
		}

		bool IsSyncSample() const
		{
			return bCurrentIsSyncSample;
		}

		uint32 GetNumTotalSamples() const
		{ return NumTotalSamples; }
		uint32 GetCurrentSampleNum() const
		{ return CurrentSampleNum; }
		bool IsFirst() const
		{ return CurrentSampleNum == 0; }
		bool IsLast() const
		{ return CurrentSampleNum + 1 >= NumTotalSamples; }

	private:
		TSharedPtr<FMP4BoxSTSS, ESPMode::ThreadSafe> StssBox;
		uint32 NumTotalSamples = 0;
		uint32 CurrentSampleNum = 0;
		uint32 CurrentEntryIndex = 0;
		bool bCurrentIsSyncSample = true;
	};

	/****************************************************************************************************************************************************/

	struct FSBGPBoxIterator
	{
		void SetBox(const TSharedPtr<FMP4BoxSBGP, ESPMode::ThreadSafe>& InSbgpBox, uint32 InDefaultSampleGroupDescriptionIndex, uint32 InMaxSamples)
		{
			NumTotalSamples = InMaxSamples;
			Current_sample_count = InMaxSamples;
			Current_group_description_index = 0;
			CurrentEntryIndex = 0;
			CurrentSampleNum = 0;
			LocalSampleInGroup = 0;
			SbgpBox = InSbgpBox;
			if (SbgpBox.IsValid())
			{
				NumDescribedSamples = SbgpBox->GetNumTotalSamples();
				DefaultSampleGroupDescriptionIndex = InDefaultSampleGroupDescriptionIndex;
				check(SbgpBox->GetEntries().Num());
				if (SbgpBox->GetEntries().Num())
				{
					const TArray<FMP4BoxSBGP::FEntry>& Entries(SbgpBox->GetEntries());
					Current_sample_count = Entries[0].sample_count;
					Current_group_description_index = Entries[0].group_description_index;
				}
			}
		}

		void SetToSampleNumber(uint32 InSampleNum)
		{
			InSampleNum = InSampleNum < NumTotalSamples ? InSampleNum : NumTotalSamples-1;

			if (!SbgpBox.IsValid())
			{
				CurrentSampleNum = InSampleNum;
				LocalSampleInGroup = InSampleNum;
				return;
			}
			// It is permitted to have fewer sample entries here than there are samples elsewhere.
			// In that case the default value is to be used.
			if (InSampleNum < NumDescribedSamples)
			{
				const TArray<FMP4BoxSBGP::FEntry>& Entries(SbgpBox->GetEntries());
				CurrentEntryIndex = 0;
				Current_sample_count = Entries[0].sample_count;
				Current_group_description_index = Entries[0].group_description_index;
				uint32 n=0;
				while(1)
				{
					n += Current_sample_count;
					if (n > InSampleNum)
					{
						LocalSampleInGroup = Current_sample_count - (n - InSampleNum);
						break;
					}
					else
					{
						check(CurrentEntryIndex < (uint32)Entries.Num());
						++CurrentEntryIndex;
						Current_sample_count = Entries[CurrentEntryIndex].sample_count;
						Current_group_description_index = Entries[CurrentEntryIndex].group_description_index;
					}
				}
			}
			else
			{
				Current_sample_count = NumTotalSamples - NumDescribedSamples;
				Current_group_description_index = DefaultSampleGroupDescriptionIndex;
				LocalSampleInGroup = InSampleNum - NumDescribedSamples;
				CurrentEntryIndex = SbgpBox->GetEntries().Num();
			}
			CurrentSampleNum = InSampleNum;
		}

		bool Next()
		{
			if (IsLast())
			{
				return false;
			}

			++CurrentSampleNum;
			++LocalSampleInGroup;
			if (!SbgpBox.IsValid() || CurrentSampleNum > NumDescribedSamples)
			{
				return true;
			}
			if (CurrentSampleNum == NumDescribedSamples)
			{
				Current_sample_count = NumTotalSamples - NumDescribedSamples;
				Current_group_description_index = DefaultSampleGroupDescriptionIndex;
				LocalSampleInGroup = CurrentSampleNum - NumDescribedSamples;
				CurrentEntryIndex = SbgpBox->GetEntries().Num();
				return true;
			}
			// Reached end of the current group run?
			if (LocalSampleInGroup == Current_sample_count)
			{
				LocalSampleInGroup = 0;
				const TArray<FMP4BoxSBGP::FEntry>& Entries(SbgpBox->GetEntries());
				check(CurrentEntryIndex < (uint32)Entries.Num());
				++CurrentEntryIndex;
				Current_sample_count = Entries[CurrentEntryIndex].sample_count;
				Current_group_description_index = Entries[CurrentEntryIndex].group_description_index;
			}
			return true;
		}

		bool Prev()
		{
			if (IsFirst())
			{
				return false;
			}

			--CurrentSampleNum;
			if (!SbgpBox.IsValid() || CurrentSampleNum >= NumDescribedSamples)
			{
				--LocalSampleInGroup;
				return true;
			}

			// At the start of the current chunk run?
			if (LocalSampleInGroup > 0)
			{
				--LocalSampleInGroup;
			}
			else
			{
				const TArray<FMP4BoxSBGP::FEntry>& Entries(SbgpBox->GetEntries());
				// Go back an entry if there is one.
				if (CurrentEntryIndex)
				{
					--CurrentEntryIndex;
					Current_sample_count = Entries[CurrentEntryIndex].sample_count;
					Current_group_description_index = Entries[CurrentEntryIndex].group_description_index;
				}
				LocalSampleInGroup = Current_sample_count - 1;
			}
			return true;
		}

		uint32 GetCurrentGroupDescriptionIndex() const
		{ return Current_group_description_index; }
		uint32 GetNumSamplesInCurrentGroup() const
		{ return Current_sample_count; }
		uint32 GetSampleIndexInCurrentGroup() const
		{ return LocalSampleInGroup; }
		uint32 GetCurrentSampleNum() const
		{ return CurrentSampleNum; }
		bool IsFirst() const
		{ return CurrentSampleNum == 0; }
		bool IsLast() const
		{ return CurrentSampleNum + 1 >= NumTotalSamples; }

	private:
		TSharedPtr<FMP4BoxSBGP, ESPMode::ThreadSafe> SbgpBox;
		uint32 Current_sample_count = 0;
		uint32 Current_group_description_index = 0;
		uint32 DefaultSampleGroupDescriptionIndex = 0;

		uint32 NumTotalSamples = 0;
		uint32 NumDescribedSamples = 0;
		uint32 CurrentSampleNum = 0;
		uint32 LocalSampleInGroup = 0;
		uint32 CurrentEntryIndex = 0;
	};

	/****************************************************************************************************************************************************/

	struct FSUBSBoxIterator
	{
		void SetBox(const TSharedPtr<FMP4BoxSUBS, ESPMode::ThreadSafe>& InSubsBox, uint32 InMaxSamples)
		{
			NumTotalSamples = InMaxSamples;
			FirstDescribedSampleNum = 0;
			LastDescribedSampleNum = 0;
			CurrentEntryIndex = 0;
			CurrentSampleNum = 0;
			SubsBox = InSubsBox;
			if (SubsBox.IsValid())
			{
				int32 NumEntries = SubsBox->GetEntries().Num();
				if (NumEntries)
				{
					FirstDescribedSampleNum = SubsBox->GetEntries()[0].GetSampleNumber();
					LastDescribedSampleNum = SubsBox->GetEntries().Last().GetSampleNumber();
				}
			}
		}

		void SetToSampleNumber(uint32 InSampleNum)
		{
			InSampleNum = InSampleNum < NumTotalSamples ? InSampleNum : NumTotalSamples-1;
			CurrentSampleNum = InSampleNum;
			const uint32 snumPlus1 = CurrentSampleNum + 1;
			if (!SubsBox.IsValid())
			{
				return;
			}
			if (snumPlus1 <= FirstDescribedSampleNum)
			{
				CurrentEntryIndex = 0;
			}
			else if (snumPlus1 > LastDescribedSampleNum)
			{
				CurrentEntryIndex = SubsBox->GetEntries().Num() ? SubsBox->GetEntries().Num() - 1 : 0;
			}
			else
			{
				// Have to locate the entry for the sample number
				const TArray<FMP4BoxSUBS::FEntry>& Entries = SubsBox->GetEntries();
				for(int32 i=0, iMax=Entries.Num(); i<iMax; ++i)
				{
					if (Entries[i].GetSampleNumber() >= snumPlus1)
					{
						CurrentEntryIndex = i;
						break;
					}
				}
			}
		}

		bool Next()
		{
			if (IsLast())
			{
				return false;
			}
			++CurrentSampleNum;
			if (SubsBox.IsValid())
			{
				if (CurrentSampleNum > LastDescribedSampleNum)
				{
					CurrentEntryIndex = SubsBox->GetEntries().Num() ? SubsBox->GetEntries().Num() - 1 : 0;
				}
				else
				{
					check(CurrentEntryIndex >= 0 && CurrentEntryIndex < SubsBox->GetEntries().Num());
					// If the current subsample list entries sample number equals the current sample
					// number we need to advance the list index. This is because the sample numbers
					// in the list are 1-based while our internal sample number is 0-based.
					// The next entry in the subsample list may not be for the next sample due to
					// the list being sparsely coded, but that is checked for in `HaveSubsamplesForCurrent()`.
					while(CurrentSampleNum >= SubsBox->GetEntries()[CurrentEntryIndex].GetSampleNumber())
					{
						if (++CurrentEntryIndex >= SubsBox->GetEntries().Num())
						{
							--CurrentEntryIndex;
							break;
						}
					}
				}
			}
			return true;
		}

		bool Prev()
		{
			if (IsFirst())
			{
				return false;
			}
			--CurrentSampleNum;
			if (SubsBox.IsValid())
			{
				if (CurrentSampleNum < FirstDescribedSampleNum)
				{
					CurrentEntryIndex = 0;
				}
				else if (CurrentEntryIndex >= 0 && CurrentEntryIndex < SubsBox->GetEntries().Num())
				{
					// See comments up in `Next()`
					while(1+CurrentSampleNum < SubsBox->GetEntries()[CurrentEntryIndex].GetSampleNumber())
					{
						if (--CurrentEntryIndex <= 0)
						{
							CurrentEntryIndex = 0;
							break;
						}
					}
				}
			}
			return true;
		}

		TOptional<uint32> GetSubsampleBoxFlags() const
		{
			return SubsBox.IsValid() ? TOptional<uint32>(SubsBox->GetFlags()) : TOptional<uint32>();
		}

		bool HaveSubsamplesForCurrent() const
		{
			if (!FirstDescribedSampleNum || (CurrentSampleNum+1 < FirstDescribedSampleNum) || (CurrentSampleNum+1 > LastDescribedSampleNum) || CurrentEntryIndex >= SubsBox->GetEntries().Num())
			{
				return false;
			}
			return CurrentSampleNum+1 == SubsBox->GetEntries()[CurrentEntryIndex].GetSampleNumber();
		}

		const FMP4BoxSUBS::FSubsampleList& GetCurrentSubsamples() const
		{
			return HaveSubsamplesForCurrent() ? SubsBox->GetEntries()[CurrentEntryIndex].GetEntries() : EmptySubsampleList;
		}

		uint32 GetCurrentSampleNum() const
		{ return CurrentSampleNum; }
		bool IsFirst() const
		{ return CurrentSampleNum == 0; }
		bool IsLast() const
		{ return CurrentSampleNum + 1 >= NumTotalSamples; }

	private:
		TSharedPtr<FMP4BoxSUBS, ESPMode::ThreadSafe> SubsBox;
		FMP4BoxSUBS::FSubsampleList EmptySubsampleList;

		uint32 NumTotalSamples = 0;
		uint32 FirstDescribedSampleNum = 0;
		uint32 LastDescribedSampleNum = 0;
		uint32 CurrentSampleNum = 0;
		int32 CurrentEntryIndex = 0;
	};

	/****************************************************************************************************************************************************/

	struct FSENCBoxIterator
	{
		void SetBox(const TSharedPtr<FMP4BoxTENC, ESPMode::ThreadSafe>& InTencBox, const TSharedPtr<FMP4BoxSENC, ESPMode::ThreadSafe>& InSencBox, uint32 InMaxSamples)
		{
			NumTotalSamples = InMaxSamples;
			CurrentSampleNum = 0;
			SencBox = InSencBox;
			if (SencBox.IsValid())
			{
				NumBoxSamples = SencBox->GetEntries().Num();
			}
			if (InTencBox.IsValid())
			{
				DefaultEntry.IV = InTencBox->GetDefaultConstantIV();
			}
		}

		void SetToSampleNumber(uint32 InSampleNum)
		{
			InSampleNum = InSampleNum < NumTotalSamples ? InSampleNum : NumTotalSamples-1;
			CurrentSampleNum = InSampleNum;
		}

		bool Next()
		{
			if (IsLast())
			{
				return false;
			}
			++CurrentSampleNum;
			return true;
		}

		bool Prev()
		{
			if (IsFirst())
			{
				return false;
			}
			--CurrentSampleNum;
			return true;
		}

		bool IsFirst() const
		{ return CurrentSampleNum == 0; }
		bool IsLast() const
		{ return CurrentSampleNum + 1 >= NumTotalSamples; }

		const FMP4BoxSENC::FEntry& GetCurrentEntry() const
		{
			return CurrentSampleNum < NumBoxSamples ? SencBox->GetEntries()[CurrentSampleNum] : DefaultEntry;
		}

	private:
		TSharedPtr<FMP4BoxSENC, ESPMode::ThreadSafe> SencBox;
		FMP4BoxSENC::FEntry DefaultEntry;
		uint32 NumTotalSamples = 0;
		uint32 NumBoxSamples = 0;
		uint32 CurrentSampleNum = 0;
	};

} // namespace MP4Boxes
