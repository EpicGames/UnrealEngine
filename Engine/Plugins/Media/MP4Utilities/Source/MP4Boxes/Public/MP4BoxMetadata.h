// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MP4Boxes.h"

namespace MP4Boxes
{

	/****************************************************************************************************************************************************/

	struct FMP4TrackMetadataCommon
	{
		FString LanguageCode;	// 3 letter ISO 639-2T
		FString LanguageTag;	// BCP47 language tag if available
		FString Name;
		FString HandlerName;
	};

} // namespace MP4Boxes
