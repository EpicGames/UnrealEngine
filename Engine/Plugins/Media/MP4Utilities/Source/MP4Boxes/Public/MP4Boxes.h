// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "Containers/ChunkedArray.h"
#include "Misc/Timespan.h"

#include "MP4BoxBase.h"

#define UE_API MP4BOXES_API

namespace MP4Boxes
{
	/**
	 * `free` and `skip` box
	 * ISO/IEC 14496-12:2022 - 8.1.2 Free space box
	 */
	class FMP4BoxFREE : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxFREE>(new FMP4BoxFREE(InParent, InBoxInfo)); }
		virtual ~FMP4BoxFREE() = default;
	protected:
		FMP4BoxFREE(const TWeakPtr<FMP4BoxBase>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
	};


	/**
	 * `ftyp` box
	 * ISO/IEC 14496-12:2022 - 4.3 File-type box
	 */
	class FMP4BoxFTYP : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxFTYP>(new FMP4BoxFTYP(InParent, InBoxInfo)); }
		virtual ~FMP4BoxFTYP() = default;
		UE_API uint32 GetMajorBrand();
		UE_API uint32 GetMinorVersion();
		UE_API TConstArrayView<uint32> GetCompatibleBrands();
		UE_API TConstArrayView<uint32> GetAllBrands();
	protected:
		FMP4BoxFTYP(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `styp` box
	 * ISO/IEC 14496-12:2022 - 8.16.2 Segment type box
	 */
	class FMP4BoxSTYP : public FMP4BoxFTYP
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxSTYP>(new FMP4BoxSTYP(InParent, InBoxInfo)); }
		virtual ~FMP4BoxSTYP() = default;
	protected:
		FMP4BoxSTYP(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxFTYP(InParent, InBoxInfo)
		{ }
	};


	/**
	 * `moov` box
	 * ISO/IEC 14496-12:2022 - 8.2.1 Movie box
	 */
	class FMP4BoxMOOV : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxMOOV>(new FMP4BoxMOOV(InParent, InBoxInfo)); }
		virtual ~FMP4BoxMOOV() = default;
		bool IsLeafBox() const override { return false; }
	protected:
		FMP4BoxMOOV(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
	};


	/**
	 * `mvhd` box
	 * ISO/IEC 14496-12:2022 - 8.2.2 Movie header box
	 */
	class FMP4BoxMVHD : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxMVHD>(new FMP4BoxMVHD(InParent, InBoxInfo)); }
		virtual ~FMP4BoxMVHD() = default;

		UE_API MP4Utilities::FFractionalTime GetDuration();
		UE_API uint32 GetTimescale();
	protected:
		FMP4BoxMVHD(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `trak` box
	 * ISO/IEC 14496-12:2022 - 8.3.1 Track box
	 */
	class FMP4BoxTRAK : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxTRAK>(new FMP4BoxTRAK(InParent, InBoxInfo)); }
		virtual ~FMP4BoxTRAK() = default;
		bool IsLeafBox() const override { return false; }
	protected:
		FMP4BoxTRAK(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
	};


	/**
	 * `tkhd` box
	 * ISO/IEC 14496-12:2022 - 8.3.2 Track header box
	 */
	class FMP4BoxTKHD : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxTKHD>(new FMP4BoxTKHD(InParent, InBoxInfo)); }
		virtual ~FMP4BoxTKHD() = default;

		UE_API uint32 GetFlags();

		bool IsEnabled()
		{ return !!(GetFlags() & 1); }

		bool IsInMovie()
		{ return !!(GetFlags() & 2); }

		bool IsInPreview()
		{ return !!(GetFlags() & 4); }

		bool IsTrackSizeAspectRatio()
		{ return !!(GetFlags() & 8); }

		/**
		 * Returns the duration of this track, measured in the timescale of the `mvhd` box.
		 * This is just the value as stored in the box. You need to apply the timescale yourself.
		 */
		UE_API int64 GetDuration();

		UE_API uint32 GetTrackID();

		UE_API uint16 GetWidth();
		UE_API uint16 GetHeight();

		UE_API TArray<int32> GetMatrix();
		UE_API uint32 GetWidthRaw();
		UE_API uint32 GetHeightRaw();
		UE_API uint16 GetVolumeRaw();
	protected:
		FMP4BoxTKHD(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `tref` box
	 * ISO/IEC 14496-12:2022 - 8.3.3 Track reference box
	 */
	class FMP4BoxTREF : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxTREF>(new FMP4BoxTREF(InParent, InBoxInfo)); }
		virtual ~FMP4BoxTREF() = default;
		bool IsLeafBox() const override { return false; }
		bool IsListOfEntries() const override { return true; }

		struct FEntry
		{
			uint32 Type = 0;
			TArray<uint32> TrackIDs;
		};

		UE_API const TArray<FEntry>& GetEntries();
		UE_API TArray<FEntry> GetEntriesOfType(uint32 InType);
	protected:
		FMP4BoxTREF(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `edts` box
	 * ISO/IEC 14496-12:2022 - 8.6.5 Edit box
	 */
	class FMP4BoxEDTS : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxEDTS>(new FMP4BoxEDTS(InParent, InBoxInfo)); }
		virtual ~FMP4BoxEDTS() = default;
		bool IsLeafBox() const override { return false; }
	protected:
		FMP4BoxEDTS(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
	};


	/**
	 * `elst` box
	 * ISO/IEC 14496-12:2022 - 8.6.6 Edit list box
	 */
	class FMP4BoxELST : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxELST>(new FMP4BoxELST(InParent, InBoxInfo)); }
		virtual ~FMP4BoxELST() = default;

		struct FEntry
		{
			// Specified in units of timescale of the `mvhd` box.
			int64 edit_duration = 0;				// A uint64 in the standard, but MP4Utilities::FFractionalTime needs an int64 for the numerator.
			// Specified in units of timescale of the `mdhd` box of this track.
			int64 media_time = 0;
			int16 media_rate_integer = 0;
			int16 media_rate_fraction = 0;
		};

		UE_API const TArray<FEntry>& GetEntries();
		UE_API bool RepeatEdits();

	protected:
		FMP4BoxELST(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `mdia` box
	 * ISO/IEC 14496-12:2022 - 8.4.1 Media box
	 */
	class FMP4BoxMDIA : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxMDIA>(new FMP4BoxMDIA(InParent, InBoxInfo)); }
		virtual ~FMP4BoxMDIA() = default;
		bool IsLeafBox() const override { return false; }
	protected:
		FMP4BoxMDIA(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
	};


	/**
	 * `mdhd` box
	 * ISO/IEC 14496-12:2022 - 8.4.2 Media header box
	 */
	class FMP4BoxMDHD : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxMDHD>(new FMP4BoxMDHD(InParent, InBoxInfo)); }
		virtual ~FMP4BoxMDHD() = default;

		UE_API MP4Utilities::FFractionalTime GetDuration();
		UE_API uint32 GetTimescale();
		UE_API FString GetLanguageCode639_2T();

	protected:
		FMP4BoxMDHD(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `hdlr` box
	 * ISO/IEC 14496-12:2022 - 8.4.3 Handler reference box
	 */
	class FMP4BoxHDLR : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxHDLR>(new FMP4BoxHDLR(InParent, InBoxInfo)); }
		virtual ~FMP4BoxHDLR() = default;

		UE_API uint32 GetHandlerType();
		UE_API FString GetHandlerName();
		UE_API uint32 GetReservedValue(int32 InIndex0to2);
	protected:
		FMP4BoxHDLR(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `minf` box
	 * ISO/IEC 14496-12:2022 - 8.4.4 Media information box
	 */
	class FMP4BoxMINF : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxMINF>(new FMP4BoxMINF(InParent, InBoxInfo)); }
		virtual ~FMP4BoxMINF() = default;
		bool IsLeafBox() const override { return false; }
	protected:
		FMP4BoxMINF(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
	};


	/**
	 * `elng` box
	 * ISO/IEC 14496-12:2022 - 8.4.6 Extended language tag
	 */
	class FMP4BoxELNG : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxELNG>(new FMP4BoxELNG(InParent, InBoxInfo)); }
		virtual ~FMP4BoxELNG() = default;

		UE_API FString GetLanguageTag();
	protected:
		FMP4BoxELNG(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `dinf` box
	 * ISO/IEC 14496-12:2022 - 8.7.1 Data information box
	 */
	class FMP4BoxDINF : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxDINF>(new FMP4BoxDINF(InParent, InBoxInfo)); }
		virtual ~FMP4BoxDINF() = default;
		bool IsLeafBox() const override { return false; }
	protected:
		FMP4BoxDINF(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
	};


	/**
	 * `dref` box
	 * ISO/IEC 14496-12:2022 - 8.7.2 Data reference box
	 */
	class FMP4BoxDREF : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxDREF>(new FMP4BoxDREF(InParent, InBoxInfo)); }
		virtual ~FMP4BoxDREF() = default;
		bool IsLeafBox() const override { return false; }
		bool IsListOfEntries() const override { return true; }
	protected:
		FMP4BoxDREF(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
	};


	/**
	 * `stbl` box
	 * ISO/IEC 14496-12:2022 - 8.5.1 Sample table box
	 */
	class FMP4BoxSTBL : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxSTBL>(new FMP4BoxSTBL(InParent, InBoxInfo)); }
		virtual ~FMP4BoxSTBL() = default;
		bool IsLeafBox() const override { return false; }
	protected:
		FMP4BoxSTBL(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
	};


	/**
	 * Sample entry base class
	 * ISO/IEC 14496-12:2022 - 8.5.2 Sample description box
	 */
	class FMP4BoxSampleEntry : public FMP4BoxBase
	{
	public:
		virtual ~FMP4BoxSampleEntry() = default;
		bool IsSampleDescription() const override
		{ return true; }

		enum class ESampleType
		{
			Video,
			Audio,
			Subtitles,
			QTFFTimecode,
			TimedMetadata,
			TimedText,
			Unsupported
		};
		virtual ESampleType GetSampleType()
		{ return ESampleType::Unsupported; }
		virtual uint16 GetDataReferenceIndex() = 0;

	protected:
		FMP4BoxSampleEntry(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
	};


	/**
	 * `btrt` box
	 * ISO/IEC 14496-12:2022 - 8.5.2 Sample description box (8.5.2.2 Bitrate box)
	 */
	class FMP4BoxBTRT : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxBTRT>(new FMP4BoxBTRT(InParent, InBoxInfo)); }
		virtual ~FMP4BoxBTRT() = default;
		UE_API uint32 GetBufferSizeDB();
		UE_API uint32 GetMaxBitrate();
		UE_API uint32 GetAverageBitrate();
	protected:
		FMP4BoxBTRT(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `srat` box
	 * ISO/IEC 14496-12:2022 - 12.2.3 AudioSampleEntry (12.2.3.2 Syntax)
	 */
	class FMP4BoxSRAT : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxSRAT>(new FMP4BoxSRAT(InParent, InBoxInfo)); }
		virtual ~FMP4BoxSRAT() = default;
		UE_API uint32 GetSampleRate();
	protected:
		FMP4BoxSRAT(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * Visual sample entry
	 * ISO/IEC 14496-12:2022 - 12.1.3 Sample entry
	 */
	class FMP4BoxVisualSampleEntry : public FMP4BoxSampleEntry
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxVisualSampleEntry>(new FMP4BoxVisualSampleEntry(InParent, InBoxInfo)); }
		virtual ~FMP4BoxVisualSampleEntry() = default;
		ESampleType GetSampleType() override
		{ return ESampleType::Video; }
		uint16 GetDataReferenceIndex() override;
		UE_API uint16 GetWidth();
		UE_API uint16 GetHeight();
		UE_API uint16 GetFrameCount();
		UE_API uint16 GetDepth();
		UE_API FString GetCompressorName();
	protected:
		FMP4BoxVisualSampleEntry(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxSampleEntry(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * Audio sample entry
	 * ISO/IEC 14496-12:2022 - 12.2.3 Sample entry
	 */
	class FMP4BoxAudioSampleEntry : public FMP4BoxSampleEntry
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxAudioSampleEntry>(new FMP4BoxAudioSampleEntry(InParent, InBoxInfo)); }
		virtual ~FMP4BoxAudioSampleEntry() = default;
		ESampleType GetSampleType() override
		{ return ESampleType::Audio; }
		uint16 GetDataReferenceIndex() override;
		UE_API uint32 GetSampleRate();
		UE_API int32 GetChannelCount();
		UE_API int32 GetSampleSize();
		UE_API bool HaveFormatSpecificFlags();
		UE_API int32 GetFormatSpecificFlags();
		UE_API uint32 GetConstBytesPerAudioPacket();
		UE_API uint32 GetConstLPCMFramesPerAudioPacket();
	protected:
		FMP4BoxAudioSampleEntry(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxSampleEntry(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * XML subtitle sample entry
	 * ISO/IEC 14496-12:2022 - 12.6.3 Sample entry
	 */
	class FMP4BoxXMLSubtitleSampleEntry : public FMP4BoxSampleEntry
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxXMLSubtitleSampleEntry>(new FMP4BoxXMLSubtitleSampleEntry(InParent, InBoxInfo)); }
		virtual ~FMP4BoxXMLSubtitleSampleEntry() = default;
		ESampleType GetSampleType() override;
		uint16 GetDataReferenceIndex() override;
	protected:
		FMP4BoxXMLSubtitleSampleEntry(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxSampleEntry(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * WebVTT sample entry
	 * ISO/IEC 14496-30:2018 - 6.5 Sample entry format
	 */
	class FMP4BoxWVTTSampleEntry : public FMP4BoxSampleEntry
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxWVTTSampleEntry>(new FMP4BoxWVTTSampleEntry(InParent, InBoxInfo)); }
		virtual ~FMP4BoxWVTTSampleEntry() = default;
		ESampleType GetSampleType() override;
		uint16 GetDataReferenceIndex() override;
	protected:
		FMP4BoxWVTTSampleEntry(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxSampleEntry(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * Timecode sample entry
	 * Apple QuickTime: https://developer.apple.com/documentation/quicktime-file-format/timecode_sample_description
	 */
	class FMP4BoxQTFFTimecodeSampleEntry : public FMP4BoxSampleEntry
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxQTFFTimecodeSampleEntry>(new FMP4BoxQTFFTimecodeSampleEntry(InParent, InBoxInfo)); }
		virtual ~FMP4BoxQTFFTimecodeSampleEntry() = default;
		ESampleType GetSampleType() override
		{ return ESampleType::QTFFTimecode; }
		uint16 GetDataReferenceIndex() override;

		// See: https://developer.apple.com/documentation/quicktime-file-format/timecode_sample_description/flags
		enum EFlags : uint32
		{
			DropFrame = 0x0001,					// Indicates whether the timecode is drop frame. Set it to 1 if the timecode is drop frame.
			Max24Hour = 0x0002,					// Indicates whether the timecode wraps after 24 hours. Set it to 1 if the timecode wraps.
			AllowNegativeTimes = 0x0004,		// Indicates whether negative time values are allowed. Set it to 1 if the timecode supports negative values.
			Counter = 0x0008					// Indicates whether the time value corresponds to a tape counter value. Set it to 1 if the timecode values are tape counter values.
		};
		UE_API uint32 GetFlags();
		UE_API uint32 GetTimescale();
		UE_API uint32 GetFrameDuration();
		UE_API uint32 GetNumberOfFrames();
	protected:
		FMP4BoxQTFFTimecodeSampleEntry(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxSampleEntry(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * 3GPP / TX3G Text sample entry (ETSI TS 126 245 V11.0.0 - 5.16 Sample Description Format)
	 */
	class FMP4BoxTX3GSampleEntry : public FMP4BoxSampleEntry
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxTX3GSampleEntry>(new FMP4BoxTX3GSampleEntry(InParent, InBoxInfo)); }
		virtual ~FMP4BoxTX3GSampleEntry() = default;
		ESampleType GetSampleType() override;
		uint16 GetDataReferenceIndex() override;
	protected:
		FMP4BoxTX3GSampleEntry(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxSampleEntry(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `pasp` box
	 * ISO/IEC 14496-12:2022 - 12.1.4 Pixel Aspect Ratio
	 */
	class FMP4BoxPASP : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxPASP>(new FMP4BoxPASP(InParent, InBoxInfo)); }
		virtual ~FMP4BoxPASP() = default;
	protected:
		FMP4BoxPASP(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
	};


	/**
	 * `colr` box
	 * ISO/IEC 14496-12:2022 - 12.1.5 Colour information
	 */
	class FMP4BoxCOLR : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxCOLR>(new FMP4BoxCOLR(InParent, InBoxInfo)); }
		virtual ~FMP4BoxCOLR() = default;
		enum class EColorType
		{
			nclx,
			rICC,
			prof,
			Unsupported
		};
		struct FColorNCLX
		{
			uint16 colour_primaries = 0;
			uint16 transfer_characteristics = 0;
			uint16 matrix_coefficients;
			uint8 full_range_flag = 0;
		};
		UE_API EColorType GetColorType();
		UE_API const FColorNCLX& GetColorNCLX();
	protected:
		FMP4BoxCOLR(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `clli` box
	 * ISO/IEC 14496-12:2022 - 12.1.6 Content light level
	 */
	class FMP4BoxCLLI : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxCLLI>(new FMP4BoxCLLI(InParent, InBoxInfo)); }
		virtual ~FMP4BoxCLLI() = default;
		//UE_API xxxx GetXXXX();
	protected:
		FMP4BoxCLLI(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `mdcv` box
	 * ISO/IEC 14496-12:2022 - 12.1.7 Mastering display colour volume
	 */
	class FMP4BoxMDCV : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxMDCV>(new FMP4BoxMDCV(InParent, InBoxInfo)); }
		virtual ~FMP4BoxMDCV() = default;
		//UE_API xxxx GetXXXX();
	protected:
		FMP4BoxMDCV(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `stsd` box
	 * ISO/IEC 14496-12:2022 - 8.5.2 Sample description box
	 */
	class FMP4BoxSTSD : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxSTSD>(new FMP4BoxSTSD(InParent, InBoxInfo)); }
		virtual ~FMP4BoxSTSD() = default;
		bool IsLeafBox() const override { return false; }
		bool IsListOfEntries() const override { return true; }

		UE_API uint8 GetBoxVersion();
		UE_API FMP4BoxSampleEntry::ESampleType GetSampleType();

		template<typename T>
		void GetSampleDescriptions(TArray<TSharedPtr<T, ESPMode::ThreadSafe>>& OutChildren)
		{
			// Get the sample type managed by this box. This may implicitly trigger parsing!
			const FMP4BoxSampleEntry::ESampleType SampleType = GetSampleType();
			for(auto& It : GetChildren())
			{
				if (It->IsSampleDescription() && StaticCastSharedPtr<FMP4BoxSampleEntry>(It)->GetSampleType() == SampleType)
				{
					uint16 DummyParseTrigger = StaticCastSharedPtr<FMP4BoxSampleEntry>(It)->GetDataReferenceIndex();
					(void)DummyParseTrigger;

					OutChildren.Emplace(StaticCastSharedPtr<T>(It));
				}
			}
		}

	protected:
		FMP4BoxSTSD(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `stts` box
	 * ISO/IEC 14496-12:2022 - 8.6.1.2 Decoding time to sample box
	 */
	class FMP4BoxSTTS : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxSTTS>(new FMP4BoxSTTS(InParent, InBoxInfo)); }
		virtual ~FMP4BoxSTTS() = default;
		struct FEntry
		{
			uint32 sample_count = 0;
			uint32 sample_delta = 0;
		};
		UE_API const TArray<FEntry>& GetEntries();
		UE_API uint32 GetNumTotalSamples();
		UE_API int64 GetTotalDuration();
		UE_API bool HasConstantSampleDuration();
		UE_API int64 GetConstantSampleDuration();
	protected:
		FMP4BoxSTTS(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `ctts` box
	 * ISO/IEC 14496-12:2022 - 8.6.1.3 Composition time to sample box
	 */
	class FMP4BoxCTTS : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxCTTS>(new FMP4BoxCTTS(InParent, InBoxInfo)); }
		virtual ~FMP4BoxCTTS() = default;
		struct FEntry
		{
			// A 64 bit value used here to hold both signed values (version 1) and 32 bit unsigned values (version 0).
			int64 sample_offset = 0;
			uint32 sample_count = 0;
		};
		UE_API uint8 GetBoxVersion();
		UE_API const TArray<FEntry>& GetEntries();
		UE_API uint32 GetNumTotalSamples();
	protected:
		FMP4BoxCTTS(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `cslg` box
	 * ISO/IEC 14496-12:2022 - 8.6.1.4 Composition to decode box
	 */
	class FMP4BoxCSLG : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxCSLG>(new FMP4BoxCSLG(InParent, InBoxInfo)); }
		virtual ~FMP4BoxCSLG() = default;
	protected:
		FMP4BoxCSLG(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
	};


	/**
	 * `stss` box
	 * ISO/IEC 14496-12:2022 - 8.6.2 Sync sample box
	 */
	class FMP4BoxSTSS : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxSTSS>(new FMP4BoxSTSS(InParent, InBoxInfo)); }
		virtual ~FMP4BoxSTSS() = default;
		UE_API const TArray<uint32>& GetEntries();
	protected:
		FMP4BoxSTSS(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `sdtp` box
	 * ISO/IEC 14496-12:2022 - 8.6.4 Independent and disposable samples box
	 */
	class FMP4BoxSDTP : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxSDTP>(new FMP4BoxSDTP(InParent, InBoxInfo)); }
		virtual ~FMP4BoxSDTP() = default;
		UE_API TConstArrayView<uint8> GetEntries();
	protected:
		FMP4BoxSDTP(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `stsc` box
	 * ISO/IEC 14496-12:2022 - 8.7.4 Sample to chunk box
	 */
	class FMP4BoxSTSC : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxSTSC>(new FMP4BoxSTSC(InParent, InBoxInfo)); }
		virtual ~FMP4BoxSTSC() = default;
		struct FEntry
		{
			uint32 first_chunk = 0;
			uint32 samples_per_chunk = 0;
			uint32 sample_description_index = 0;
		};
		UE_API const TArray<FEntry>& GetEntries();
	protected:
		FMP4BoxSTSC(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `stsz` / `stz2` box
	 * ISO/IEC 14496-12:2022 - 8.7.3 Sample size boxes
	 */
	class FMP4BoxSTSZ : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxSTSZ>(new FMP4BoxSTSZ(InParent, InBoxInfo)); }
		virtual ~FMP4BoxSTSZ() = default;
		uint32 GetType() const override
		{
			// This class handles both `stsz` and `stz2`.
			// Since `stz2` is rare and most of the time we expect `stsz` we always return
			// this type, so user code does not need to worry about the difference.
			return MP4Utilities::MakeBoxAtom('s','t','s','z');
		}
		UE_API uint32 GetNumberOfSamples();
		UE_API uint32 GetSizeOfSample(uint32 InIndex);
		UE_API bool HasConstantSampleSize();
		UE_API uint32 GetConstantSampleSize();
	protected:
		FMP4BoxSTSZ(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `stco` box, `co64` box
	 * ISO/IEC 14496-12:2022 - 8.7.5 Chunk offset box
	 */
	class FMP4BoxSTCO : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxSTCO>(new FMP4BoxSTCO(InParent, InBoxInfo)); }
		virtual ~FMP4BoxSTCO() = default;
		uint32 GetType() const override
		{
			// This class handles both `stco` and `co64`.
			// For simplicities sake, when searching for the chunk offset box, we pretend this is
			// an `stco` at all times, so user code does not need to worry about the difference.
			return MP4Utilities::MakeBoxAtom('s','t','c','o');
		}
		UE_API uint32 GetNumberOfEntries();
		UE_API uint64 GetChunkOffset(uint32 InIndex);
	protected:
		FMP4BoxSTCO(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `subs` box
	 * ISO/IEC 14496-12:2022 - 8.7.7 Sub-sample information box
	 */
	class FMP4BoxSUBS : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxSUBS>(new FMP4BoxSUBS(InParent, InBoxInfo)); }
		virtual ~FMP4BoxSUBS() = default;

		UE_API uint32 GetFlags();

		struct FSubsample
		{
			uint32 codec_specific_parameters = 0;
			uint32 subsample_size = 0;
			uint32 subsample_offset = 0;	// calculated for convenience.
			uint8 subsample_priority = 0;
			uint8 discardable = 0;
		};
		using FSubsampleList = TChunkedArray<FSubsample, 16>;
		struct FEntry
		{
			uint32 FirstSampleNumber = 0;	// Sample number starts at 1, so 0 is not valid.
			FSubsampleList SubSamples;
			uint32 GetSampleNumber() const
			{ return FirstSampleNumber; }
			const FSubsampleList& GetEntries() const
			{ return SubSamples; }
		};
		UE_API const TArray<FEntry>& GetEntries();

	protected:
		FMP4BoxSUBS(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `saiz` box
	 * ISO/IEC 14496-12:2022 - 8.7.8 Sample auxiliary information sizes box
	 */
	class FMP4BoxSAIZ : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxSAIZ>(new FMP4BoxSAIZ(InParent, InBoxInfo)); }
		virtual ~FMP4BoxSAIZ() = default;
		UE_API bool HasAuxInfoType();
		UE_API uint32 GetAuxInfoType();
		UE_API uint32 GetAuxInfoTypeParameter();
		UE_API uint32 GetSampleCount();
		UE_API uint8 GetSampleInfoSize(uint32 InForSampleNumber);
	protected:
		FMP4BoxSAIZ(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `saio` box
	 * ISO/IEC 14496-12:2022 - 8.7.9 Sample auxiliary information offsets box
	 */
	class FMP4BoxSAIO : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxSAIO>(new FMP4BoxSAIO(InParent, InBoxInfo)); }
		virtual ~FMP4BoxSAIO() = default;
		UE_API bool HasAuxInfoType();
		UE_API uint32 GetAuxInfoType();
		UE_API uint32 GetAuxInfoTypeParameter();
		UE_API uint32 GetEntryCount();
		UE_API uint64 GetEntry(uint32 InForEntryNumber);
	protected:
		FMP4BoxSAIO(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `sgpd` box
	 * ISO/IEC 14496-12:2022 - 8.9.3 Sample group description box
	 */
	class FMP4BoxSGPD : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxSGPD>(new FMP4BoxSGPD(InParent, InBoxInfo)); }
		virtual ~FMP4BoxSGPD() = default;

		UE_API uint32 GetGroupingType();
		UE_API const TArray<TConstArrayView<uint8>>& GetGroupDescriptionEntries();
		UE_API uint32 GetDefaultGroupDescriptionIndex();
	protected:
		FMP4BoxSGPD(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `sbgp` box
	 * ISO/IEC 14496-12:2022 - 8.9.2 Sample to group box
	 */
	class FMP4BoxSBGP : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxSBGP>(new FMP4BoxSBGP(InParent, InBoxInfo)); }
		virtual ~FMP4BoxSBGP() = default;

		struct FEntry
		{
			uint32 sample_count = 0;
			uint32 group_description_index = 0;
		};

		UE_API uint32 GetGroupingType();
		UE_API uint32 GetGroupingTypeParameter();
		UE_API const TArray<FEntry>& GetEntries();
		UE_API uint32 GetNumTotalSamples();
	protected:
		FMP4BoxSBGP(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `mvex` box
	 * ISO/IEC 14496-12:2022 - 8.8.1 Movie extends box
	 */
	class FMP4BoxMVEX : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxMVEX>(new FMP4BoxMVEX(InParent, InBoxInfo)); }
		virtual ~FMP4BoxMVEX() = default;
		bool IsLeafBox() const override { return false; }
	protected:
		FMP4BoxMVEX(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
	};


	/**
	 * `mehd` box
	 * ISO/IEC 14496-12:2022 - 8.8.2 Movie extends header box
	 */
	class FMP4BoxMEHD : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxMEHD>(new FMP4BoxMEHD(InParent, InBoxInfo)); }
		virtual ~FMP4BoxMEHD() = default;
		/**
		 * Returns the fragment duration in the timescale of the `mvhd` box.
		 * This is just the value as stored in the box. You need to apply the timescale yourself.
		 */
		UE_API int64 GetFragmentDuration();
	protected:
		FMP4BoxMEHD(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `trex` box
	 * ISO/IEC 14496-12:2022 - 8.8.3 Track extends box
	 */
	class FMP4BoxTREX : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxTREX>(new FMP4BoxTREX(InParent, InBoxInfo)); }
		virtual ~FMP4BoxTREX() = default;
		UE_API uint32 GetTrackID();
		UE_API uint32 GetDefaultSampleDescriptionIndex();
		UE_API uint32 GetDefaultSampleDuration();
		UE_API uint32 GetDefaultSampleSize();
		UE_API uint32 GetDefaultSampleFlags();
	protected:
		FMP4BoxTREX(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `leva` box
	 * ISO/IEC 14496-12:2022 - 8.8.13 Level assignment box
	 */
	class FMP4BoxLEVA : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxLEVA>(new FMP4BoxLEVA(InParent, InBoxInfo)); }
		virtual ~FMP4BoxLEVA() = default;
	protected:
		FMP4BoxLEVA(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
	};


	/**
	 * `prft` box
	 * ISO/IEC 14496-12:2022 - 8.16.5 Producer reference time box
	 */
	class FMP4BoxPRFT : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxPRFT>(new FMP4BoxPRFT(InParent, InBoxInfo)); }
		virtual ~FMP4BoxPRFT() = default;

		enum class EReferenceType
		{
			Undefined,
			Encoder,		// Input or output. If distinction is needed, look at the flags.
			Writer,			// Fragment written or finalized. If distinction is needed, look at the flags.
			Captured,		// Flags == 8|16
			Application		// Flags == 8 (also Flags == 16 despite this not being defined to be set alone)
		};

		UE_API uint32 GetFlags();
		UE_API uint32 GetReferenceTrackID();
		UE_API uint64 GetNTPTimestamp();
		UE_API uint64 GetMediaTime();
		EReferenceType GetReferenceType()
		{
			uint32 Flg = GetFlags();
			if (Flg == 0 || Flg == 1) return EReferenceType::Encoder;
			else if (Flg == 2 || Flg == 4) return EReferenceType::Writer;
			else if (Flg == 8 || Flg == 16) return EReferenceType::Application;
			else if (Flg == 24) return EReferenceType::Captured;
			return EReferenceType::Undefined;
		}
	protected:
		FMP4BoxPRFT(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `moof` box
	 * ISO/IEC 14496-12:2022 - 8.8.4 Movie fragment box
	 */
	class FMP4BoxMOOF : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxMOOF>(new FMP4BoxMOOF(InParent, InBoxInfo)); }
		virtual ~FMP4BoxMOOF() = default;
		bool IsLeafBox() const override { return false; }
	protected:
		FMP4BoxMOOF(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
	};


	/**
	 * `mfhd` box
	 * ISO/IEC 14496-12:2022 - 8.8.5 Movie fragment header box
	 */
	class FMP4BoxMFHD : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxMFHD>(new FMP4BoxMFHD(InParent, InBoxInfo)); }
		virtual ~FMP4BoxMFHD() = default;
		UE_API uint32 GetSequenceNumber();
	protected:
		FMP4BoxMFHD(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `traf` box
	 * ISO/IEC 14496-12:2022 - 8.8.6 Track fragment box
	 */
	class FMP4BoxTRAF : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxTRAF>(new FMP4BoxTRAF(InParent, InBoxInfo)); }
		virtual ~FMP4BoxTRAF() = default;
		bool IsLeafBox() const override { return false; }
	protected:
		FMP4BoxTRAF(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
	};


	/**
	 * `tfhd` box
	 * ISO/IEC 14496-12:2022 - 8.8.7 Track fragment header box
	 */
	class FMP4BoxTFHD : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxTFHD>(new FMP4BoxTFHD(InParent, InBoxInfo)); }
		virtual ~FMP4BoxTFHD() = default;

		UE_API uint32 GetTrackID();
		UE_API bool HasBaseDataOffset();
		UE_API uint64 GetBaseDataOffset();
		UE_API bool HasSampleDescriptionIndex();
		UE_API uint32 GetSampleDescriptionIndex();
		UE_API bool HasDefaultSampleDuration();
		UE_API uint32 GetDefaultSampleDuration();
		UE_API bool HasDefaultSampleSize();
		UE_API uint32 GetDefaultSampleSize();
		UE_API bool HasDefaultSampleFlags();
		UE_API uint32 GetDefaultSampleFlags();
		UE_API bool IsDurationEmpty();
		UE_API bool IsMoofDefaultBase();
	protected:
		FMP4BoxTFHD(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `trun` box
	 * ISO/IEC 14496-12:2022 - 8.8.8 Track fragment run box
	 */
	class FMP4BoxTRUN : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxTRUN>(new FMP4BoxTRUN(InParent, InBoxInfo)); }
		virtual ~FMP4BoxTRUN() = default;
		UE_API uint32 GetNumberOfSamples();
		UE_API bool HasSampleOffset();
		UE_API int32 GetSampleOffset();
		UE_API bool HasFirstSampleFlags();
		UE_API uint32 GetFirstSampleFlags();
		UE_API bool HasSampleDurations();
		UE_API const TArray<uint32>& GetSampleDurations();
		UE_API bool HasSampleSizes();
		UE_API const TArray<uint32>& GetSampleSizes();
		UE_API int64 GetTotalSampleSize();
		UE_API bool HasSampleFlags();
		UE_API const TArray<uint32>& GetSampleFlags();
		UE_API bool HasSampleCompositionTimeOffsets();
		UE_API const TArray<int32>& GetSampleCompositionTimeOffsets();
		UE_API int64 GetTotalDuration();
	protected:
		FMP4BoxTRUN(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `mfra` box
	 * ISO/IEC 14496-12:2022 - 8.8.9 Movie fragment random access box
	 */
	class FMP4BoxMFRA : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxMFRA>(new FMP4BoxMFRA(InParent, InBoxInfo)); }
		virtual ~FMP4BoxMFRA() = default;
		bool IsLeafBox() const override { return false; }
	protected:
		FMP4BoxMFRA(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
	};


	/**
	 * `tfra` box
	 * ISO/IEC 14496-12:2022 - 8.8.10 Track fragment random access box
	 */
	class FMP4BoxTFRA : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxTFRA>(new FMP4BoxTFRA(InParent, InBoxInfo)); }
		virtual ~FMP4BoxTFRA() = default;
	protected:
		FMP4BoxTFRA(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
	};


	/**
	 * `mfro` box
	 * ISO/IEC 14496-12:2022 - 8.8.11 Movie fragment random access offset box
	 */
	class FMP4BoxMFRO : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxMFRO>(new FMP4BoxMFRO(InParent, InBoxInfo)); }
		virtual ~FMP4BoxMFRO() = default;
	protected:
		FMP4BoxMFRO(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
	};


	/**
	 * `tfdt` box
	 * ISO/IEC 14496-12:2022 - 8.8.12 Track fragment decode time box
	 */
	class FMP4BoxTFDT : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxTFDT>(new FMP4BoxTFDT(InParent, InBoxInfo)); }
		virtual ~FMP4BoxTFDT() = default;
		UE_API uint64 GetBaseMediaDecodeTime();
	protected:
		FMP4BoxTFDT(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `sidx` box
	 * ISO/IEC 14496-12:2022 - 8.16.3 Segment index box
	 */
	class FMP4BoxSIDX : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxSIDX>(new FMP4BoxSIDX(InParent, InBoxInfo)); }
		virtual ~FMP4BoxSIDX() = default;
		struct FEntry
		{
			uint32 SubSegmentDuration;
			uint32 IsReferenceType : 1;
			uint32 Size : 31;
			uint32 StartsWithSAP : 1;
			uint32 SAPType : 3;
			uint32 SAPDeltaTime : 28;
		};
		using FEntryList = TChunkedArray<FEntry, 1024>;
		UE_API uint32 GetReferenceID();
		UE_API uint32 GetTimescale();
		UE_API uint64 GetEarliestPresentationTime();
		UE_API uint64 GetFirstOffset();
		UE_API const FEntryList& GetEntries();
	protected:
		FMP4BoxSIDX(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `ssix` box
	 * ISO/IEC 14496-12:2022 - 8.16.4 Subsegment index box
	 */
	class FMP4BoxSSIX : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxSSIX>(new FMP4BoxSSIX(InParent, InBoxInfo)); }
		virtual ~FMP4BoxSSIX() = default;
	protected:
		FMP4BoxSSIX(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
	};


	/**
	 * `vmhd` box
	 * ISO/IEC 14496-12:2022 - 12.1.2 Video media header
	 */
	class FMP4BoxVMHD : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxVMHD>(new FMP4BoxVMHD(InParent, InBoxInfo)); }
		virtual ~FMP4BoxVMHD() = default;
	protected:
		FMP4BoxVMHD(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
	};


	/**
	 * `smhd` box
	 * ISO/IEC 14496-12:2022 - 12.2.2 Sound media header
	 */
	class FMP4BoxSMHD : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxSMHD>(new FMP4BoxSMHD(InParent, InBoxInfo)); }
		virtual ~FMP4BoxSMHD() = default;
	protected:
		FMP4BoxSMHD(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
	};


	/**
	 * `sthd` box
	 * ISO/IEC 14496-12:2022 - 12.6.2 Subtitle media header
	 */
	class FMP4BoxSTHD : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxSTHD>(new FMP4BoxSTHD(InParent, InBoxInfo)); }
		virtual ~FMP4BoxSTHD() = default;
	protected:
		FMP4BoxSTHD(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
	};


	/**
	 * `nmhd` box
	 * ISO/IEC 14496-12:2022 - 8.4.5.2 Null media header box
	 */
	class FMP4BoxNMHD : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxNMHD>(new FMP4BoxNMHD(InParent, InBoxInfo)); }
		virtual ~FMP4BoxNMHD() = default;
	protected:
		FMP4BoxNMHD(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
	};


	/**
	 * `gmhd` box
	 * Apple QuickTime: https://developer.apple.com/documentation/quicktime-file-format/base_media_information_header_atom
	 */
	class FMP4BoxGMHD : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxGMHD>(new FMP4BoxGMHD(InParent, InBoxInfo)); }
		virtual ~FMP4BoxGMHD() = default;
	protected:
		FMP4BoxGMHD(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
	};


	/**
	 * `udta` box
	 * ISO/IEC 14496-12:2022 - 8.10.1 User data box
	 */
	class FMP4BoxUDTA : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxUDTA>(new FMP4BoxUDTA(InParent, InBoxInfo)); }
		virtual ~FMP4BoxUDTA() = default;
		bool IsLeafBox() const override { return false; }
	protected:
		FMP4BoxUDTA(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
	};


	/**
	 * `meta` box
	 * ISO/IEC 14496-12:2022 - 8.11.1 MetaBox
	 */
	class FMP4BoxMETA : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxMETA>(new FMP4BoxMETA(InParent, InBoxInfo)); }
		virtual ~FMP4BoxMETA() = default;
		bool IsLeafBox() const override { return false; }
		bool IsListOfEntries() const override { return true; }
		UE_API TSharedPtr<FMP4BoxHDLR, ESPMode::ThreadSafe> GetHandler();
	protected:
		FMP4BoxMETA(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `sinf` box
	 * ISO/IEC 14496-12:2022 - 8.12.2 Protection scheme information box
	 */
	class FMP4BoxSINF : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxSINF>(new FMP4BoxSINF(InParent, InBoxInfo)); }
		virtual ~FMP4BoxSINF() = default;
		bool IsLeafBox() const override { return false; }
	protected:
		FMP4BoxSINF(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
	};


	/**
	 * `frma` box
	 * ISO/IEC 14496-12:2022 - 8.12.3 Original format box
	 */
	class FMP4BoxFRMA : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxFRMA>(new FMP4BoxFRMA(InParent, InBoxInfo)); }
		virtual ~FMP4BoxFRMA() = default;
		UE_API uint32 GetOriginalFormat();
	protected:
		FMP4BoxFRMA(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `schm` box
	 * ISO/IEC 14496-12:2022 - 8.12.6 Scheme type box
	 */
	class FMP4BoxSCHM : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxSCHM>(new FMP4BoxSCHM(InParent, InBoxInfo)); }
		virtual ~FMP4BoxSCHM() = default;
		UE_API uint32 GetSchemeType();
		UE_API uint32 GetSchemeVersion();
	protected:
		FMP4BoxSCHM(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `schi` box
	 * ISO/IEC 14496-12:2022 - 8.12.7 Scheme information box
	 */
	class FMP4BoxSCHI : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxSCHI>(new FMP4BoxSCHI(InParent, InBoxInfo)); }
		virtual ~FMP4BoxSCHI() = default;
		bool IsLeafBox() const override { return false; }
	protected:
		FMP4BoxSCHI(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
	};


	/**
	 * `pssh` box
	 * ISO/IEC 23001-7:2023 - 8.1 Protection system specific header box
	 */
	class FMP4BoxPSSH : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxPSSH>(new FMP4BoxPSSH(InParent, InBoxInfo)); }
		virtual ~FMP4BoxPSSH() = default;
		UE_API TConstArrayView<uint8> GetSystemID();
		UE_API const TArray<TConstArrayView<uint8>>& GetKIDs();
		UE_API TConstArrayView<uint8> GetData();
	protected:
		FMP4BoxPSSH(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `tenc` box
	 * ISO/IEC 23001-7:2023 - 8.2 Track Encryption box
	 */
	class FMP4BoxTENC : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxTENC>(new FMP4BoxTENC(InParent, InBoxInfo)); }
		virtual ~FMP4BoxTENC() = default;
		UE_API bool HasDefaultCryptBlockValues();
		UE_API uint8 GetDefaultCryptByteBlock();
		UE_API uint8 GetDefaultSkipByteBlock();
		UE_API uint8 GetDefaultIsProtected();
		UE_API uint8 GetDefaultPerSampleIVSize();
		UE_API TConstArrayView<uint8> GetDefaultKID();
		UE_API TConstArrayView<uint8> GetDefaultConstantIV();
	protected:
		FMP4BoxTENC(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `senc` box
	 * ISO/IEC 23001-7:2023 - 7.2.1 Sample encryption box - Definition
	 */
	class FMP4BoxSENC : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxSENC>(new FMP4BoxSENC(InParent, InBoxInfo)); }
		virtual ~FMP4BoxSENC() = default;

		UE_API void Prepare(TWeakPtr<FMP4BoxTENC, ESPMode::ThreadSafe> InTENCBox);

		struct FSubSample
		{
			uint32 NumEncryptedBytes = 0;
			uint16 NumClearBytes = 0;
		};
		struct FEntry
		{
			TArray<uint8> IV;
			TArray<FSubSample> SubSamples;
		};
		UE_API const TArray<FEntry>& GetEntries();
	protected:
		FMP4BoxSENC(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
		TWeakPtr<FMP4BoxTENC, ESPMode::ThreadSafe> ParentTENCBox;
	};


	/**
	 * `emsg` box
	 * ISO/IEC 23009-1:2022 - 5.10.3.3 Event message box
	 */
	class FMP4BoxEMSG : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxEMSG>(new FMP4BoxEMSG(InParent, InBoxInfo)); }
		virtual ~FMP4BoxEMSG() = default;

		UE_API uint8 GetVersion();	// version 0: GetPresentationTimeDelta(); version 1: GetPresentationTime()
		UE_API FString GetSchemeIdUri();
		UE_API FString GetValue();
		UE_API uint32 GetTimescale();
		UE_API uint32 GetPresentationTimeDelta();
		UE_API uint64 GetPresentationTime();
		UE_API uint32 GetEventDuration();
		UE_API uint32 GetID();
		UE_API TConstArrayView<uint8> GetMessageData();
	protected:
		FMP4BoxEMSG(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `avcC` box
	 * ISO/IEC 14496-15:2022 - 5.4.2 AVC video stream definition
	 */
	class FMP4BoxAVCC : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxAVCC>(new FMP4BoxAVCC(InParent, InBoxInfo)); }
		virtual ~FMP4BoxAVCC() = default;
		UE_API TConstArrayView<uint8> GetAVCDecoderConfigurationRecord();
	protected:
		FMP4BoxAVCC(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `hvcC` box
	 * ISO/IEC 14496-15:2022 - 8.3 Sample and configuration definition
	 */
	class FMP4BoxHVCC : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxHVCC>(new FMP4BoxHVCC(InParent, InBoxInfo)); }
		virtual ~FMP4BoxHVCC() = default;
		UE_API TConstArrayView<uint8> GetHEVCDecoderConfigurationRecord();
	protected:
		FMP4BoxHVCC(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `vpcC` box
	 * VP Codec ISO Media File Format Binding
	 * See: https://www.webmproject.org/vp9/mp4/#syntax_1
	 */
	class FMP4BoxVPCC : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxVPCC>(new FMP4BoxVPCC(InParent, InBoxInfo)); }
		virtual ~FMP4BoxVPCC() = default;
		UE_API TConstArrayView<uint8> GetVPCodecConfigurationRecord();
	protected:
		FMP4BoxVPCC(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `av1C` box
	 * AV1 Codec ISO Media File Format Binding
	 * See: https://aomediacodec.github.io/av1-isobmff/#av1codecconfigurationbox-syntax
	 */
	class FMP4BoxAV1C : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxAV1C>(new FMP4BoxAV1C(InParent, InBoxInfo)); }
		virtual ~FMP4BoxAV1C() = default;
		UE_API TConstArrayView<uint8> GetAV1CodecConfigurationRecord();
	protected:
		FMP4BoxAV1C(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `apvC` box
	 * APV Codec ISO Media File Format Binding
	 * See: https://github.com/AcademySoftwareFoundation/openapv/blob/main/readme/apv_isobmff.md#apv-codec-configuration-box
	 */
	class FMP4BoxAPVC : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxAPVC>(new FMP4BoxAPVC(InParent, InBoxInfo)); }
		virtual ~FMP4BoxAPVC() = default;
		UE_API TConstArrayView<uint8> GetAVPCCodecConfigurationRecord();
	protected:
		FMP4BoxAPVC(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `iods` box
	 * ISO/IEC 14496-14:2020 - 6.2 Object Descriptor Box
	 */
	class FMP4BoxIODS : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxIODS>(new FMP4BoxIODS(InParent, InBoxInfo)); }
		virtual ~FMP4BoxIODS() = default;
		UE_API TConstArrayView<uint8> GetObjectDescriptor();
	protected:
		FMP4BoxIODS(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `esds` box
	 * ISO/IEC 14496-14:2020 - 6.7.2
	 */
	class FMP4BoxESDS : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxESDS>(new FMP4BoxESDS(InParent, InBoxInfo)); }
		virtual ~FMP4BoxESDS() = default;
		UE_API TConstArrayView<uint8> GetESDescriptor();
	protected:
		FMP4BoxESDS(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `dvcC` box, `dvvC` box, `dvwC` box
	 * Dolby Vision Streams Within the ISO Base:2023 - 2.2 Dolby Vision configuration boxes
	 */
	class FMP4BoxDVCC : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxDVCC>(new FMP4BoxDVCC(InParent, InBoxInfo)); }
		virtual ~FMP4BoxDVCC() = default;
		UE_API TConstArrayView<uint8> GetDOVIDecoderConfigurationRecord();
	protected:
		FMP4BoxDVCC(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `dac3` box
	 * Annex F.4 of ETSI TS 102 366 - AC3SpecificBox
	 */
	class FMP4BoxDAC3 : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxDAC3>(new FMP4BoxDAC3(InParent, InBoxInfo)); }
		virtual ~FMP4BoxDAC3() = default;
		UE_API TConstArrayView<uint8> GetAC3SpecificBox();
	protected:
		FMP4BoxDAC3(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `dec3` box
	 * Annex F.6 of ETSI TS 102 366 - EC3SpecificBox
	 */
	class FMP4BoxDEC3 : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxDEC3>(new FMP4BoxDEC3(InParent, InBoxInfo)); }
		virtual ~FMP4BoxDEC3() = default;
		UE_API TConstArrayView<uint8> GetEC3SpecificBox();
	protected:
		FMP4BoxDEC3(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `dfLa` box
	 * FLAC (https://github.com/xiph/flac/blob/master/doc/isoflac.txt) specific box - 3.3.2 FLAC Specific Box
	 */
	class FMP4BoxDFLA : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxDFLA>(new FMP4BoxDFLA(InParent, InBoxInfo)); }
		virtual ~FMP4BoxDFLA() = default;
		UE_API TConstArrayView<uint8> GetFLACSpecificBox();
	protected:
		FMP4BoxDFLA(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `dOps` box
	 * Opus (https://opus-codec.org/docs/opus_in_isobmff.html#4.3.2) specific box - 4.3.2 Opus Specific Bo
	 */
	class FMP4BoxDOPS : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxDOPS>(new FMP4BoxDOPS(InParent, InBoxInfo)); }
		virtual ~FMP4BoxDOPS() = default;
		UE_API TConstArrayView<uint8> GetOpusSpecificBox();
	protected:
		FMP4BoxDOPS(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
		void ParseIfRequired();
		struct FParsed;
		TPimplPtr<FParsed> Parsed;
	};


	/**
	 * `wave` box
	 * Apple Quicktime: https://developer.apple.com/documentation/quicktime-file-format/sidecompressionparam_atom
	 */
	class FMP4BoxWAVE : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxWAVE>(new FMP4BoxWAVE(InParent, InBoxInfo)); }
		virtual ~FMP4BoxWAVE() = default;
		bool IsLeafBox() const override { return false; }
	protected:
		FMP4BoxWAVE(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
	};


	/**
	 * `tapt` box
	 * Apple Quicktime: https://developer.apple.com/documentation/quicktime-file-format/track_aperture_mode_dimensions_atom
	 */
	class FMP4BoxTAPT : public FMP4BoxBase
	{
	public:
		static TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
		{ return MakeShareable<FMP4BoxTAPT>(new FMP4BoxTAPT(InParent, InBoxInfo)); }
		virtual ~FMP4BoxTAPT() = default;
		bool IsLeafBox() const override { return false; }
	protected:
		FMP4BoxTAPT(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo)
			: FMP4BoxBase(InParent, InBoxInfo)
		{ }
	};


	class FMP4BoxFactory
	{
	public:
		static UE_API FMP4BoxFactory& Get();
		UE_API TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe> Create(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& InParent, const MP4Utilities::FMP4BoxInfo& InBoxInfo);
	private:
		FMP4BoxFactory();
		TMap<uint32, TSharedPtr<FMP4BoxBase, ESPMode::ThreadSafe>(*)(const TWeakPtr<FMP4BoxBase, ESPMode::ThreadSafe>& /*InParent*/, const MP4Utilities::FMP4BoxInfo& /*InBoxInfo*/)> FactoryMap;
	};

} // namespace MP4Boxes

#undef UE_API
