// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "Misc/Timespan.h"
#include "Misc/Optional.h"
#include "MP4Boxes.h"
#include "MP4BoxIterators.h"
#include "MP4BoxMetadata.h"

#define UE_API MP4BOXES_API

namespace MP4Boxes
{
	class FMP4Track : public TSharedFromThis<FMP4Track>
	{
	public:
		struct FFirstSample
		{
			int64 SamplePTS = 0;
			int64 StartPTS = 0;
			uint32 SampleNumber = 0;
			uint32 SyncSampleNumber = 0;
		};

		struct FLastSample
		{
			int64 SamplePTS = 0;
			int64 EndPTS = 0;
			uint32 SampleNumber = 0;
			uint32 LastSampleNumber = 0;
		};

	protected:
		struct FConvenience
		{
			FFirstSample FirstSample;
			FLastSample LastSample;
			MP4Utilities::FFractionalTime FullMovieDuration;
			MP4Utilities::FFractionalTime DurationFromMvhdBox;
			MP4Utilities::FFractionalTime DurationFromTkhdBox;
			MP4Utilities::FFractionalTime DurationFromMdhdBox;
			MP4Utilities::FFractionalTime MappedDurationFromElstBox;
			int64 CompositionTimeAtZeroPoint = 0;
			int64 DTSShiftAtZeroPoint = 0;
			uint32 TrackID = 0;
			uint32 NumTotalSamples = 0;
			uint32 NumMoovSamples = 0;
		};

	public:
		class FFragmentInfo
		{
		public:
			struct FTrackTrafs
			{
				TSharedPtr<FMP4BoxMOOF, ESPMode::ThreadSafe> MoofBox;
				TSharedPtr<FMP4BoxTRAF, ESPMode::ThreadSafe> TrafBox;
				TSharedPtr<FMP4BoxTFHD, ESPMode::ThreadSafe> TfhdBox;
				TSharedPtr<FMP4BoxSENC, ESPMode::ThreadSafe> SencBox;
				TArray<TSharedPtr<FMP4BoxTRUN, ESPMode::ThreadSafe>> TrunBoxes;
				TArray<TSharedPtr<FMP4BoxSUBS, ESPMode::ThreadSafe>> SubsBoxes;
				TOptional<uint64> BaseMediaDecodeTime;
				uint32 AbsoluteFirstSampleNumber = 0;
				int64 FirstTrackTotalDTS = 0;
				int64 LastTrackTotalDTSAndDur = 0;
				int64 Duration = 0;
				uint32 NumSamples = 0;
			};
			TArray<TSharedPtr<FTrackTrafs, ESPMode::ThreadSafe>> TrackFragments;
			TSharedPtr<FMP4BoxTKHD, ESPMode::ThreadSafe> TkhdBox;
			TSharedPtr<FMP4BoxMDHD, ESPMode::ThreadSafe> MdhdBox;
			TSharedPtr<FMP4BoxTREX, ESPMode::ThreadSafe> TrexBox;
			TSharedPtr<FMP4BoxTENC, ESPMode::ThreadSafe> TencBox;
			uint32 NumTotalFragmentSamples = 0;

			bool HasBeenPrepared() const
			{ return bHasBeenPrepared;}
			UE_API bool Prepare(const TSharedPtr<FMP4BoxTRAK, ESPMode::ThreadSafe>& InTrakBox, const TSharedPtr<FMP4BoxMVEX, ESPMode::ThreadSafe>& InMvexBox, const TArray<TSharedPtr<FMP4BoxBase>>& InAllRootBoxes);
			UE_API int64 GetTotalDuration();
		private:
			void CalculateDurations();
			int64 BaseTrackDuration = 0;
			int64 SumOfFragmentDurations = 0;
			uint32 NumBaseTrackSamples = 0;
			bool bDidCalculateDuration = false;
			bool bHasBeenPrepared = false;
		};

		static TSharedPtr<FMP4Track, ESPMode::ThreadSafe> Create(const TSharedPtr<FMP4BoxTRAK, ESPMode::ThreadSafe>& InTrakBox, const TSharedPtr<FMP4Track::FFragmentInfo, ESPMode::ThreadSafe>& InFragmentInfo)
		{ return MakeShareable<FMP4Track>(new FMP4Track(InTrakBox, InFragmentInfo)); }
		virtual ~FMP4Track() = default;

		UE_API bool Prepare(MP4Utilities::FFractionalTime InFullMovieDuration, MP4Utilities::FFractionalTime InAdjustedMovieDuration);
		UE_API FString GetLastError();
		UE_API const FMP4TrackMetadataCommon& GetCommonMetadata();

		class FIterator
		{
		public:
			virtual ~FIterator() = default;

			bool IsValid() const
			{ return Convs.NumTotalSamples != 0; }
			TWeakPtr<FMP4Track, ESPMode::ThreadSafe> GetTrack() const
			{ return Track; }
			uint32 GetTrackID() const
			{ return Convs.TrackID; }
			uint32 GetSampleNumber() const
			{ return SampleNumber; }
			// Returns the DTS without mapping to the timeline.
			MP4Utilities::FFractionalTime GetDTS() const
			{ return CurrentDTS; }
			// Returns the effective DTS, which has the timeline mapping applied. This may result in a negative value.
			MP4Utilities::FFractionalTime GetEffectiveDTS() const
			{ return CurrentEffectiveDTS; }
			// Returns the PTS as the sum of the DTS and the composition time offset, without mapping to the timeline.
			MP4Utilities::FFractionalTime GetPTS() const
			{ return CurrentPTS; }
			// Returns the effective PTS, which is the media time mapped into the 0-based timeline.
			MP4Utilities::FFractionalTime GetEffectivePTS() const
			{ return CurrentEffectivePTS; }
			MP4Utilities::FFractionalTime GetDuration() const
			{ return CurrentDuration; }
			// Returns the duration as an FTimespan, which may be slightly more accurate than as a fraction.
			FTimespan GetDurationAsTimespan() const
			{ return CurrentDurationTS; }
			bool IsSyncOrRAPSample() const
			{ return bCurrentIsSyncOrRAP; }
			int64 GetSampleSize() const
			{ return CurrentSampleSize; }
			int64 GetSampleFileOffset() const
			{ return CurrentSampleFileOffset; }
			uint32 GetTimescale() const
			{ return Convs.DurationFromMdhdBox.GetDenominator(); }
			uint32 GetNumSamples() const
			{ return Convs.NumTotalSamples; }
			// Returns the track's entire media duration, not affected by an edit list. Timescale comes from `mdhd` box.
			MP4Utilities::FFractionalTime GetTrackDuration() const
			{ return Convs.DurationFromMdhdBox; }
			// Returns the effective track's duration, as specified by an edit list. Timescale has been converted into `mdhd` timescale!
			MP4Utilities::FFractionalTime GetEffectiveTrackDuration() const
			{ return Convs.MappedDurationFromElstBox; }
			// Returns the base media decode time for the current fragment.
			TOptional<MP4Utilities::FFractionalTime> GetBaseMediaDecodeTime() const
			{ return CurrentBaseMediaDecodeTime; }
			// Returns the flags from the `subs` full box. Unset if there is no subsample box.
			TOptional<uint32> GetSubsampleBoxFlags() const
			{ return CurrentSubsampleBoxFlags; }
			// Returns the list of subsamples for this sample, if there are any.
			const FMP4BoxSUBS::FSubsampleList& GetSubsampleList() const
			{ return CurrentSubsampleList; }
			// Returns the encryption information for this sample.
			const FMP4BoxSENC::FEntry& GetEncryptionInfo() const
			{ return EncryptionEntry; } 

			// Advances this iterator to the next sample. Returns true if there is one, false if not.
			// This iterates over the entire track, ignoring timeline mapping.
			UE_API bool Next();

			// Recedes this iterator to the previous sample. Returns true if there is one, false if not.
			// This iterates over the entire track, ignoring timeline mapping.
			UE_API bool Prev();

			// Returns whether the iterator points to the first overall sample, ignoring mapping to the timeline.
			bool IsFirst() const
			{ return SampleNumber == 0; }
			// Returns whether the iterator points to the last overall sample, ignoring mapping to the timeline.
			bool IsLast() const
			{ return SampleNumber+1 >= Convs.NumTotalSamples; }

			// Same as above, but obeying the timeline mapping and taking into consideration
			// any required earlier sync frame and later frames due to reordering.
			UE_API bool NextEffective();
			UE_API bool PrevEffective();
			bool IsFirstEffective() const
			{ return SampleNumber <= Convs.FirstSample.SyncSampleNumber; }
			bool IsLastEffective() const
			{ return SampleNumber >= Convs.LastSample.LastSampleNumber; }

			struct FChunkInfo
			{
				uint32 NumSamples = 0;
				int64 SizeInBytes = 0;
				int64 Duration = 0;
				bool bIsLastChunk = false;
			};
			// Returns information about the samples that are remaining in the current chunk.
			// This is intended only for forward playback, not reverse.
			UE_API bool GetCurrentChunkRemainingSampleInfo(FChunkInfo& OutInfo) const;

			// Creates a copy of this iterator.
			TSharedPtr<FIterator, ESPMode::ThreadSafe> Clone()
			{ return MakeShareable(new FIterator(*this)); }
		protected:
			class FTrafTrunIterator
			{
			public:
				FTrafTrunIterator() = default;
				void SetFragmentInfo(const TSharedPtr<FFragmentInfo, ESPMode::ThreadSafe>& InFragmentInfo);
				bool IsSetup() const
				{ return FragmentInfo.IsValid(); }

				bool StartAt(uint32 InSampleNum);
				bool Prev();
				bool Next();

				int64 GetStartDTS() const
				{ return TrunStartDTS; }
				uint32 GetAbsoluteSampleNumber() const
				{ return FirstAbsoluteSampleNumber + (uint32)SampleNumber; }
				int64 GetAbsoluteOffset() const
				{ return AbsoluteSampleByteOffset; }
				int32 GetSampleNumber() const
				{ return SampleNumber; }
				int64 GetSampleSize() const
				{ return SampleSize; }
				int64 GetDTS() const
				{ return DTS; }
				int64 GetPTS() const
				{ return PTS; }
				int64 GetDuration() const
				{ return Duration; }
				bool IsSyncSample() const
				{ return (SampleFlags & 0x00010000U) == 0; }
				TOptional<uint64> GetBaseMediaDecodeTime() const
				{ return BaseMediaDecodeTime; }

				TOptional<uint32> GetSubsampleBoxFlags() const
				{ return subsIt.GetSubsampleBoxFlags(); }
				const FMP4BoxSUBS::FSubsampleList& GetSubsampleList() const
				{ return SubsampleList; }

				const FMP4BoxSENC::FEntry& GetEncryptionInfo() const
				{ return EncryptionEntry; } 

				bool GetCurrentTrafRemainingSampleInfo(FChunkInfo& OutInfo) const;
			private:
				void Update(const TSharedPtr<FMP4BoxTRUN, ESPMode::ThreadSafe>& InTrun);
				void Update(const TSharedPtr<FFragmentInfo::FTrackTrafs, ESPMode::ThreadSafe>& InTraf);
				TSharedPtr<FFragmentInfo, ESPMode::ThreadSafe> FragmentInfo;
				uint32 TrexDefaultSampleDescriptionIndex = 0;
				uint32 TrexDefaultSampleDuration = 0;
				uint32 TrexDefaultSampleSize = 0;
				uint32 TrexDefaultSampleFlags = 0;

				TOptional<uint64> BaseMediaDecodeTime;
				int64 TrunStartDTS = 0;
				int64 BaseOffset = 0;
				int64 DataOffset = 0;
				uint32 FirstAbsoluteSampleNumber = 0;
				uint32 DefaultSampleDescriptionIndex = 0;
				uint32 DefaultSampleDuration = 0;
				uint32 DefaultSampleSize = 0;
				uint32 DefaultSampleFlags = 0;
				int32 TrafNumber = -1;
				int32 TrunNumber = -1;
				int32 SampleNumber = -1;

				uint32 SampleFlags = 0;
				int64 AbsoluteSampleByteOffset = 0;
				int64 SampleSize = 0;
				int64 DTS = 0;
				int64 PTS = 0;
				int64 Duration = 0;

				int64 TrunFirstSampleAbsoluteByteOffset = -1;
				int64 TrunLastSampleAbsoluteByteOffset = -1;

				FSUBSBoxIterator subsIt;
				FMP4BoxSUBS::FSubsampleList SubsampleList;

				FSENCBoxIterator sencIt;
				FMP4BoxSENC::FEntry EncryptionEntry;
			};

			FIterator() = default;
			FIterator(const FIterator& InOther) = default;
			FIterator& operator = (const FIterator&) = delete;
			friend class FMP4Track;

			void StartAt(uint32 InFrameNumber);
			void Update();

			TWeakPtr<FMP4Track, ESPMode::ThreadSafe> Track;
			FConvenience Convs;
			FSTSZBoxIterator stszIt;
			FSTTSBoxIterator sttsIt;
			FCTTSBoxIterator cttsIt;
			FSTSCBoxIterator stscIt;
			FSTSSBoxIterator stssIt;
			FSTCOBoxIterator stcoIt;
			FSBGPBoxIterator rapIt;
			FSUBSBoxIterator subsIt;
			FSENCBoxIterator sencIt;
			uint32 SampleNumber = 0;

			struct FFragmentVars
			{
				FTrafTrunIterator TrafTrunIterator;
				bool bInFragments = false;
			};
			FFragmentVars FragmentVars;

			MP4Utilities::FFractionalTime CurrentDTS;
			MP4Utilities::FFractionalTime CurrentPTS;
			MP4Utilities::FFractionalTime CurrentEffectiveDTS;
			MP4Utilities::FFractionalTime CurrentEffectivePTS;
			MP4Utilities::FFractionalTime CurrentDuration;
			TOptional<MP4Utilities::FFractionalTime> CurrentBaseMediaDecodeTime;
			FTimespan CurrentDurationTS;
			int64 CurrentSampleFileOffset = 0;
			int64 CurrentSampleSize = 0;
			bool bCurrentIsSyncOrRAP = false;
			FMP4BoxSUBS::FSubsampleList CurrentSubsampleList;
			TOptional<uint32> CurrentSubsampleBoxFlags;
			FMP4BoxSENC::FEntry EncryptionEntry;
		};

		// Create an interator starting at the first sample.
		UE_API TSharedPtr<FIterator, ESPMode::ThreadSafe> CreateIterator();
		// Create an interator starting at the last sample (used when iterating in reverse, crossing back from the beginning to the end)
		UE_API TSharedPtr<FIterator, ESPMode::ThreadSafe> CreateIteratorAtLastFrame();
		// Create an iterator starting at a keyframe on or before the given time, or at a later time within the given
		// threshold should one be right after the given time and would not be selected due to timescale rounding issues.
		UE_API TSharedPtr<FIterator, ESPMode::ThreadSafe> CreateIteratorAtKeyframe(FTimespan InForTime, FTimespan InLaterTimeThreshold);
		// Create an interator starting at a given sample number.
		UE_API TSharedPtr<FIterator, ESPMode::ThreadSafe> CreateIterator(uint32 InAtSampleNumber);

		// Returns the number of samples in this track.
		UE_API uint32 GetNumberOfSamples();

		// Returns information about the first sample that is mapped to the 0-based timeline via `elst` box.
		const FFirstSample& GetFirstSampleInfo() const
		{ return Convs.FirstSample; }
		// Returns information about the last sample that is mapped to the 0-based timeline via `elst` box.
		const FLastSample& GetLastSampleInfo() const
		{ return Convs.LastSample; }

		// Returns the duration of the movie as a whole, which is set from the longest track.
		const MP4Utilities::FFractionalTime& GetFullMovieDuration() const
		{ return Convs.FullMovieDuration; }

		// Returns the duration of this track as mapped onto the media internal timeline (edit list).
		const MP4Utilities::FFractionalTime& GetMappedTrackDuration() const
		{ return Convs.MappedDurationFromElstBox; }
		// Returns the duration of this track media.
		const MP4Utilities::FFractionalTime& GetTrackDuration() const
		{ return Convs.DurationFromMdhdBox; }

	protected:
		FMP4Track(const TSharedPtr<FMP4BoxTRAK, ESPMode::ThreadSafe>& InTrakBox, const TSharedPtr<FMP4Track::FFragmentInfo, ESPMode::ThreadSafe>& InFragmentInfo)
			: TrakBox(InTrakBox)
			, FragmentInfo(InFragmentInfo)
		{ }

		TSharedPtr<FMP4BoxTRAK, ESPMode::ThreadSafe> TrakBox;
		TSharedPtr<FMP4Track::FFragmentInfo, ESPMode::ThreadSafe> FragmentInfo;

		TSharedPtr<FMP4BoxTKHD, ESPMode::ThreadSafe> TkhdBox;
		TSharedPtr<FMP4BoxELST, ESPMode::ThreadSafe> ElstBox;
		TSharedPtr<FMP4BoxMDHD, ESPMode::ThreadSafe> MdhdBox;
		TSharedPtr<FMP4BoxSTTS, ESPMode::ThreadSafe> SttsBox;
		TSharedPtr<FMP4BoxCTTS, ESPMode::ThreadSafe> CttsBox;
		TSharedPtr<FMP4BoxSTSC, ESPMode::ThreadSafe> StscBox;
		TSharedPtr<FMP4BoxSTSZ, ESPMode::ThreadSafe> StszBox;
		TSharedPtr<FMP4BoxSTCO, ESPMode::ThreadSafe> StcoBox;
		TSharedPtr<FMP4BoxSTSS, ESPMode::ThreadSafe> StssBox;
		TSharedPtr<FMP4BoxTENC, ESPMode::ThreadSafe> TencBox;
		TSharedPtr<FMP4BoxSENC, ESPMode::ThreadSafe> SencBox;
		TSharedPtr<FMP4BoxUDTA, ESPMode::ThreadSafe> UdtaBox;

		TArray<TSharedPtr<FMP4BoxSUBS, ESPMode::ThreadSafe>> SubsBoxes;
		TArray<TSharedPtr<FMP4BoxSGPD, ESPMode::ThreadSafe>> SgpdBoxes;
		TArray<TSharedPtr<FMP4BoxSBGP, ESPMode::ThreadSafe>> SbgpBoxes;

		FConvenience Convs;

		FMP4TrackMetadataCommon CommonMetadata;

		bool bHasBeenPrepared = false;

		FString LastErrorMessage;
	};

} // namespace MP4Boxes

#undef UE_API
