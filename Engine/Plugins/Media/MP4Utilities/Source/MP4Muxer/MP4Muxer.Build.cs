// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class MP4Muxer : ModuleRules
	{
		public MP4Muxer(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(new string[] {
				"Core",
                "MP4Utilities"
            });
		}
	}
}
