// Copyright Epic Games, Inc. All Rights Reserved.

#include "MP4Muxer.h"

#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformAffinity.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "HAL/Event.h"
#include "HAL/PlatformProcess.h"
#include "HAL/CriticalSection.h"
#include "Templates/UniquePtr.h"
#include "Containers/Queue.h"
#include "Misc/DateTime.h"
#include "Math/BigInt.h"

#include "MP4MuxerModule.h"
#include "MP4Utilities.h"
#include "MP4MuxerBoxes.h"

#include <numeric>


class FMP4RawMuxer : public IMP4RawMuxer, public FRunnable
{
public:
	virtual ~FMP4RawMuxer();

	bool Configure(const FConfiguration& InConfiguration) override;
	int32 AddTrack(const FTrackSpec& InTrackSpec) override;
	bool AddTrackReference(int32 InTrackIndexToAddReferenceTo, int32 InTrackIndexBeingReferenced, uint32 InReferenceType) override;
	bool Start(FRequestTrackDataDelegate InSampleRequestDelegate, FStatusDelegate InStatusDelegate) override;
	bool AddTrackSample(int32 InTrack, const FTrackSample& InTrackSample) override;
	bool StopAndClose() override;
	FString GetLastError() override
	{ return LastError; }
private:
	using FReferencedTrackIndexList = TArray<int32>;
	using FTrackReferences = TMap<uint32 /*ReferenceType*/, FReferencedTrackIndexList>;

	struct FTrackMuxBoxes
	{
		MP4MuxerBoxes::FBoxTREX* TREX = nullptr;
		MP4MuxerBoxes::FBoxTKHD* TKHD = nullptr;
		MP4MuxerBoxes::FBoxMDHD* MDHD = nullptr;
		MP4MuxerBoxes::FBoxSTTS* STTS = nullptr;
		MP4MuxerBoxes::FBoxSTSS* STSS = nullptr;
		MP4MuxerBoxes::FBoxCTTS* CTTS = nullptr;
		MP4MuxerBoxes::FBoxSTSC* STSC = nullptr;
		MP4MuxerBoxes::FBoxSTSZ* STSZ = nullptr;
		MP4MuxerBoxes::FBoxSTCO* STCO = nullptr;
		MP4MuxerBoxes::FBoxSUBS* SUBS = nullptr;
		MP4MuxerBoxes::FBoxELST* ELST = nullptr;
	};

	struct FTrackChunkSamples
	{
		void Empty()
		{
			Samples.Empty();
			// Subtract the wanted interleave duration from the amount accumulated to account for
			// the gathered samples having a longer duration than asked for. We need to keep the
			// remainder so the next run may collect one (or however many) fewer samples to prevent
			// the individual track chunks from not drifting apart.
			AccumulatedDuration -= InterleaveDurationNeeded;
			SumOfSampleDurations = 0;
		}
		struct FSample
		{
			TArray<FTrackSample::FSubSampleInfo> SubSamples;
			TArray64<uint8> Data;
			int64 DTS;
			int64 PTS;
			uint32 Duration;
			bool bIsKeyframe;
		};
		FTrackMuxBoxes TrackMuxBoxes;
		TArray<TUniquePtr<FSample>> Samples;
		int64 TotalTrackDuration = 0;
		int64 AccumulatedDuration = 0;
		int64 SumOfSampleDurations = 0;
		int64 InterleaveDurationNeeded = 0;
		uint32 Timescale = 0;
		uint32 NextSampleNumToRequest = 0;
		bool bGotLastSample = false;
		bool bHasFinished = false;
	};

	bool SetError(const FString& InErrorMsg)
	{
		FScopeLock lock(&Lock);
		LastError = InErrorMsg;
		bFailed = true;
		return false;
	}
	bool SetWriteError()
	{
		return SetError(FString::Printf(TEXT("Could not write to file \"%s\"."), *FilenameWritingNow));
	}
	bool SetUpdateError(const TCHAR* InWhat, const TCHAR* InBox)
	{
		return SetError(FString::Printf(TEXT("Failed to update %s in `%s` box, value now needs 64 bits!"), InWhat, InBox));
	}
	uint32 Run() override;
	void CloseFile();
	bool RequestSampleData();
	bool WriteFileStart();
	bool WriteAccumulatedSamplesStandard();
	bool WriteAccumulatedSamplesFragmented();
	bool WriteAccumulatedSamples();
	bool WriteFileEnd();
	bool WriteUpdate(int64 InFilePosition, TConstArrayView<uint8> InData);
	bool RewriteWebOptimizedFile(bool bInForceComplete = false);
	void ClearUpdateBoxes();

	FConfiguration Configuration;
	TArray<FTrackSpec> TrackSpecs;
	TMap<int32, FTrackReferences> TrackReferences;
	FRequestTrackDataDelegate SampleRequestDelegate;
	FStatusDelegate StatusDelegate;
	FString FilenameWritingNow;
	FString LastError;
	bool bFailed = false;
	bool bTerminate = false;
	FRunnableThread* Thread = nullptr;
	FEvent* WorkerEvent = nullptr;
	IFileHandle* FileHandle = nullptr;
	int64 FileOffsetOfStandardMDAT = -1;
	int64 FileEndOffsetOfStandardMDAT = -1;
	uint32 FragmentSequenceNumber = 0;
	bool bWroteMOOV = false;

	FCriticalSection Lock;

	TArray<FTrackChunkSamples> TrackChunkSamples;
	int32 MinimumMajorISOBrand = 0;
	int32 WrittenMajorISOBrand = 0;
	TUniquePtr<MP4MuxerBoxes::FBoxFTYP> FTYP;
	TUniquePtr<MP4MuxerBoxes::FBoxMOOV> MOOV;
	MP4MuxerBoxes::FBoxMVHD* MVHD = nullptr;
	MP4MuxerBoxes::FBoxMEHD* MEHD = nullptr;

	TArray<FSampleRequest> TracksAwaitingSamples;
	MP4MuxerBoxes::FBoxBase::FRewriteBoxChangeDelegate RewriteChangeDelegate;
};

TSharedRef<IMP4RawMuxer, ESPMode::ThreadSafe> IMP4RawMuxer::Create()
{
	return MakeShared<FMP4RawMuxer, ESPMode::ThreadSafe>();
}

TArray<uint8> IMP4RawMuxer::WrapDataInBox(uint32 InBoxAtom, TConstArrayView<uint8> InBoxData)
{
	TArray<uint8> Wrap;
	Wrap.SetNumUninitialized(InBoxData.Num() + 8);
	uint32* bd = reinterpret_cast<uint32*>(Wrap.GetData());
	*bd++ = MP4MuxerBoxes::EndianSwap((uint32)(InBoxData.Num() + 8));
	*bd++ = MP4MuxerBoxes::EndianSwap(InBoxAtom);
	FMemory::Memcpy(bd, InBoxData.GetData(), InBoxData.Num());
	return Wrap;
}

TArray<uint8> IMP4RawMuxer::WrapDataInBox(uint32 InBoxAtom, uint8 InBoxVersion, uint32 InBoxFlags, TConstArrayView<uint8> InBoxData)
{
	TArray<uint8> Wrap;
	Wrap.SetNumUninitialized(InBoxData.Num() + 12);
	uint32* bd = reinterpret_cast<uint32*>(Wrap.GetData());
	*bd++ = MP4MuxerBoxes::EndianSwap((uint32)(InBoxData.Num() + 12));
	*bd++ = MP4MuxerBoxes::EndianSwap(InBoxAtom);
	*bd++ = MP4MuxerBoxes::EndianSwap((uint32)((InBoxFlags & 0x00ffffff) | ((uint32)InBoxVersion << 24)));
	FMemory::Memcpy(bd, InBoxData.GetData(), InBoxData.Num());
	return Wrap;
}



FMP4RawMuxer::~FMP4RawMuxer()
{
	StopAndClose();
}

void FMP4RawMuxer::CloseFile()
{
	if (FileHandle)
	{
		delete FileHandle;
		FileHandle = nullptr;
	}
}

bool FMP4RawMuxer::Configure(const FConfiguration& InConfiguration)
{
	Configuration = InConfiguration;
	if (Configuration.InterleaveDuration.GetTicks() < 0)
	{
		return SetError(FString::Printf(TEXT("Invalid interleave duration of %.4f msec."), Configuration.InterleaveDuration.GetTotalSeconds()));
	}
	if (Configuration.InterleaveDuration.GetTicks() == 0)
	{
		Configuration.InterleaveDuration = FTimespan::FromMilliseconds(500);
	}
	if (Configuration.MuxMode == FConfiguration::EMuxMode::WebOptimized && Configuration.TemporaryFilename.IsEmpty())
	{
		return SetError(FString::Printf(TEXT("WebOptimized mode requires a temporary file.")));
	}

	RewriteChangeDelegate.BindRaw(this, &FMP4RawMuxer::WriteUpdate);
	return true;
}

int32 FMP4RawMuxer::AddTrack(const FTrackSpec& InTrackSpec)
{
	// Some basic checks
	if (!InTrackSpec.SampleEntryFormat)
	{
		SetError(TEXT("Unset sample entry format"));
		return -1;
	}
	if (!InTrackSpec.Timescale)
	{
		SetError(TEXT("Unset time scale"));
		return -1;
	}

	TrackSpecs.Emplace(InTrackSpec);
	return TrackSpecs.Num() - 1;
}

bool FMP4RawMuxer::AddTrackReference(int32 InTrackIndexToAddReferenceTo, int32 InTrackIndexBeingReferenced, uint32 InReferenceType)
{
	if (InTrackIndexToAddReferenceTo < 0 || InTrackIndexToAddReferenceTo >= TrackSpecs.Num())
	{
		SetError(TEXT("Invalid track index to add a reference to"));
		return false;
	}
	if (InTrackIndexBeingReferenced < 0 || InTrackIndexBeingReferenced >= TrackSpecs.Num())
	{
		SetError(TEXT("Invalid index of referenced track"));
		return false;
	}
	FTrackReferences& IdxRef = TrackReferences.FindOrAdd(InTrackIndexToAddReferenceTo);
	FReferencedTrackIndexList& RefList = IdxRef.FindOrAdd(InReferenceType);
	if (RefList.Contains(InTrackIndexBeingReferenced))
	{
		SetError(TEXT("Track has already been referenced before"));
		return false;
	}
	RefList.Emplace(InTrackIndexBeingReferenced);
	return true;
}




bool FMP4RawMuxer::Start(FRequestTrackDataDelegate InSampleRequestDelegate, FStatusDelegate InStatusDelegate)
{
	// If already in error just return.
	if (LastError.Len())
	{
		return false;
	}
	// Need to have a sample retrieval delegate.
	if (!InSampleRequestDelegate.IsBound() || !InStatusDelegate.IsBound())
	{
		return SetError(TEXT("Unbound delegate passed to Start()"));
	}
	// Don't call Start() twice.
	if (SampleRequestDelegate.IsBound())
	{
		return SetError(TEXT("Start() has already been called before"));
	}
	SampleRequestDelegate = InSampleRequestDelegate;
	StatusDelegate = InStatusDelegate;

	// Set up the per track sample chunks
	TrackChunkSamples.SetNum(TrackSpecs.Num());
	for(int32 i=0; i<TrackChunkSamples.Num(); ++i)
	{
		TrackChunkSamples[i].Timescale = TrackSpecs[i].Timescale;
		TrackChunkSamples[i].InterleaveDurationNeeded = MP4Utilities::ConvertToTimescale(TrackSpecs[i].Timescale, Configuration.InterleaveDuration.GetTicks(), (uint32)ETimespan::TicksPerSecond);
		if (!TrackChunkSamples[i].InterleaveDurationNeeded)
		{
			return SetError(FString::Printf(TEXT("Interleave duration not representable by a timescale of %u"), TrackSpecs[i].Timescale));
		}
	}

	WorkerEvent = FPlatformProcess::GetSynchEventFromPool(true);
	if (!WorkerEvent)
	{
		return SetError(TEXT("Start() failed to create worker event"));
	}
	Thread = FRunnableThread::Create(this, TEXT("MP4Muxer"));
	if (!Thread)
	{
		return SetError(TEXT("Start() failed to create worker thread"));
	}
	return true;
}


bool FMP4RawMuxer::StopAndClose()
{
	bTerminate = true;
	if (Thread)
	{
		Thread->WaitForCompletion();
		delete Thread;
		Thread = nullptr;
	}
	if (WorkerEvent)
	{
		FPlatformProcess::ReturnSynchEventToPool(WorkerEvent);
		WorkerEvent = nullptr;
	}
	return true;
}

bool FMP4RawMuxer::WriteFileStart()
{
	// Write the starting `ftyp` box and a size-reserving `free` box followed by the `mdat` when writing a standard file.
	MP4MuxerBoxes::FBoxBase* FileLevelBoxes = new MP4MuxerBoxes::FBoxBase(0);
	FTYP = MakeUnique<MP4MuxerBoxes::FBoxFTYP>();
	WrittenMajorISOBrand = 2;
	FTYP->SetMajorBrand(MP4MuxerBoxes::BoxAtom('i','s','o','2'));
	FTYP->SetMinorVersion(1);
	FTYP->AddCompatibleBrand(MP4MuxerBoxes::BoxAtom('a','v','c','1'));	// `iso2` (or better) requires this
	FTYP->AddCompatibleBrand(MP4MuxerBoxes::BoxAtom('m','p','4','2'));
	FileLevelBoxes->AddChild(FTYP.Get());
	MinimumMajorISOBrand = WrittenMajorISOBrand;
	MP4MuxerBoxes::FBoxMDAT* MDAT = nullptr;
	if (Configuration.MuxMode != IMP4RawMuxer::FConfiguration::EMuxMode::Fragmented)
	{
		// We may need additional 8 bytes if the following `mdat` exceeds 4GiB,
		// so we insert a `free` box with no additional zero bytes as this box size is 8 bytes.
		MP4MuxerBoxes::FBoxFREE* FREE = new MP4MuxerBoxes::FBoxFREE;
		FREE->SetNumberOfEmptyBytes(0);
		FileLevelBoxes->AddChild(FREE);
		MDAT = new MP4MuxerBoxes::FBoxMDAT;
		FileLevelBoxes->AddChild(MDAT);
	}
	// Build the binary representation of the boxes
	MP4MuxerBoxes::FBoxBuilderData builderData;
	FileLevelBoxes->BuildData(builderData);
	FileLevelBoxes->CalculateBoxSizes();
	bool bWriteOk = FileLevelBoxes->VisitForWriting([&](MP4MuxerBoxes::FBoxBase* InBox)
	{
		InBox->SetAbsoluteFileOffset(FileHandle->Tell());
		return InBox->GetCompiledBoxData().Num() ? FileHandle->Write(InBox->GetCompiledBoxData().GetData(), InBox->GetCompiledBoxData().Num()) : true;
	});
	if (!bWriteOk)
	{
		return SetWriteError();
	}
	FileOffsetOfStandardMDAT = MDAT ? MDAT->GetAbsoluteFileOffset() : -1;
	// We need to keep the ftyp box!
	FileLevelBoxes->RemoveChild(FTYP.Get());
	delete FileLevelBoxes;
	FileLevelBoxes = nullptr;

	// Create the `moov` box and the track structures.
	// We need these when writing standard as well as fragmented files, so we can create them here.
	MOOV = MakeUnique<MP4MuxerBoxes::FBoxMOOV>();
	MVHD = new MP4MuxerBoxes::FBoxMVHD;
	MP4MuxerBoxes::FBoxMVEX* MVEX = Configuration.MuxMode == IMP4RawMuxer::FConfiguration::EMuxMode::Fragmented ? new MP4MuxerBoxes::FBoxMVEX : nullptr;
	MOOV->AddChild(MVHD);
	uint64 UtcCreationDate(static_cast<uint64>((FDateTime::UtcNow() - FDateTime(1904, 1, 1)).GetTotalSeconds()));
	MVHD->SetCreationTime(UtcCreationDate);
	MVHD->SetModificationTime(UtcCreationDate);
	// Start out with a millisecond timescale for the movie box.
	MVHD->SetTimescale(1000);
	MVHD->SetDuration(0);
	MVHD->SetNextTrackID(TrackSpecs.Num() + 1);
	if (MVEX)
	{
		MOOV->AddChild(MVEX);
		/*
			We patch the `mvhd` duration at the end so there is no need to write an `mehd` box

			MEHD = new MP4MuxerBoxes::FBoxMEHD;
			MVEX->AddChild(MEHD);
		*/
	}

	for(int32 TrackIdx=0; TrackIdx<TrackSpecs.Num(); ++TrackIdx)
	{
		bool bIsVideo = false;
		bool bIsAudio = false;
		bool bIsTimecode = false;
		bool bIsOpaque = false;

		MP4MuxerBoxes::FBoxTRAK* TRAK = new MP4MuxerBoxes::FBoxTRAK;
		MOOV->AddChild(TRAK);
		MP4MuxerBoxes::FBoxTKHD* TKHD = new MP4MuxerBoxes::FBoxTKHD;
		TrackChunkSamples[TrackIdx].TrackMuxBoxes.TKHD = TKHD;
		TRAK->AddChild(TKHD);
		TKHD->SetCreationTime(UtcCreationDate);
		TKHD->SetModificationTime(UtcCreationDate);
		TKHD->SetDuration(0);
		TKHD->SetTrackID(1U + TrackIdx);
		TKHD->SetTrackEnabled(TrackSpecs[TrackIdx].bEnabled ? 1 : 0);
		TKHD->SetTrackInMovie(TrackSpecs[TrackIdx].bInMovie ? 1 : 0);
		TKHD->SetTrackInPreview(0);
		TKHD->SetLayer(TrackSpecs[TrackIdx].Layer);
		TKHD->SetAlternateGroup(TrackSpecs[TrackIdx].AlternateGroup);

		if (TrackSpecs[TrackIdx].Properties.IsType<FTrackSpec::FVideo>())
		{
			auto const& vid = TrackSpecs[TrackIdx].Properties.Get<FTrackSpec::FVideo>();
			TKHD->SetWidth((uint16)vid.DisplayWidth);
			TKHD->SetHeight((uint16)vid.DisplayHeight);
			TKHD->SetVolume(0);
			bIsVideo = true;
		}
		else if (TrackSpecs[TrackIdx].Properties.IsType<FTrackSpec::FAudio>())
		{
			//auto const& aud = TrackSpecs[TrackIdx].Properties.Get<FTrackSpec::FAudio>();
			TKHD->SetVolume(1);
			bIsAudio = true;
		}
		else if (TrackSpecs[TrackIdx].Properties.IsType<FTrackSpec::FTimecode>())
		{
			bIsTimecode = true;
		}
		else if (TrackSpecs[TrackIdx].Properties.IsType<FTrackSpec::FOpaqueData>())
		{
			auto const& op = TrackSpecs[TrackIdx].Properties.Get<FTrackSpec::FOpaqueData>();
			TKHD->SetMatrix(op.matrix);
			TKHD->SetWidthRaw(op.width);
			TKHD->SetHeightRaw(op.height);
			TKHD->SetVolumeRaw(op.volume);
			TKHD->SetCustomFlags(op.flags);
			bIsOpaque = true;
		}
		else
		{
			unimplemented();
		}

		// Does this track reference anything?
		const FTrackReferences* TrackRefs = TrackReferences.Find(TrackIdx);
		if (TrackRefs)
		{
			MP4MuxerBoxes::FBoxTREF* TREF = new MP4MuxerBoxes::FBoxTREF;
			TRAK->AddChild(TREF);
			for(auto &TrRefIt : *TrackRefs)
			{
				MP4MuxerBoxes::FBoxTREFType* TREFType = new MP4MuxerBoxes::FBoxTREFType;
				TREFType->SetReferenceType(TrRefIt.Key);
				for(int32 i=0, iMax=TrRefIt.Value.Num(); i<iMax; ++i)
				{
					TREFType->AddReferencedTrackNumber(1 + TrRefIt.Value[i]);
				}
				TREF->AddChild(TREFType);
			}
		}

		// TREX ?
		if (MVEX)
		{
			MP4MuxerBoxes::FBoxTREX* TREX = new MP4MuxerBoxes::FBoxTREX;
			TrackChunkSamples[TrackIdx].TrackMuxBoxes.TREX = TREX;
			MVEX->AddChild(TREX);
			TREX->SetTrackID(1U + TrackIdx);
			TREX->SetDefaultSampleDuration(TrackSpecs[TrackIdx].ConstantSampleDuration);
			// Constant size not used yet.
			TREX->SetDefaultSampleSize(0);	// TrackSpecs[TrackIdx].ConstantSampleSize);
			if (TrackSpecs[TrackIdx].bIsAllKeyframes)
			{
				TREX->SetDefaultSampleFlags(0x0);
			}
			else
			{
				// sample_is_non_sync_sample = 1
				TREX->SetDefaultSampleFlags(0x10000);
			}
		}

		// MDIA
		MP4MuxerBoxes::FBoxMDIA* MDIA = new MP4MuxerBoxes::FBoxMDIA;
		TRAK->AddChild(MDIA);
		// MDHD
		MP4MuxerBoxes::FBoxMDHD* MDHD = new MP4MuxerBoxes::FBoxMDHD;
		TrackChunkSamples[TrackIdx].TrackMuxBoxes.MDHD = MDHD;
		MDIA->AddChild(MDHD);
		MDHD->SetCreationTime(UtcCreationDate);
		MDHD->SetModificationTime(UtcCreationDate);
		MDHD->SetDuration(0);
		MDHD->SetTimescale(TrackSpecs[TrackIdx].Timescale);
		MDHD->SetLanguage(TrackSpecs[TrackIdx].Language[0], TrackSpecs[TrackIdx].Language[1], TrackSpecs[TrackIdx].Language[2]);

		// HDLR
		MP4MuxerBoxes::FBoxHDLR* HDLR = new MP4MuxerBoxes::FBoxHDLR;
		MDIA->AddChild(HDLR);
		// MINF
		MP4MuxerBoxes::FBoxMINF* MINF = new MP4MuxerBoxes::FBoxMINF;
		MDIA->AddChild(MINF);
		if (bIsVideo)
		{
			HDLR->SetHandlerType(MP4MuxerBoxes::BoxAtom('v','i','d','e'));
			HDLR->SetName(TEXT("VideoHandler"));
			MP4MuxerBoxes::FBoxVMHD* VMHD = new MP4MuxerBoxes::FBoxVMHD;
			MINF->AddChild(VMHD);
		}
		else if (bIsAudio)
		{
			HDLR->SetHandlerType(MP4MuxerBoxes::BoxAtom('s','o','u','n'));
			HDLR->SetName(TEXT("SoundHandler"));
			MP4MuxerBoxes::FBoxSMHD* SMHD = new MP4MuxerBoxes::FBoxSMHD;
			MINF->AddChild(SMHD);
		}
		else if (bIsTimecode)
		{
			HDLR->SetHandlerType(MP4MuxerBoxes::BoxAtom('t','m','c','d'));
			HDLR->SetName(TEXT("TimeCodeHandler"));
			MP4MuxerBoxes::FBoxNMHD* NMHD = new MP4MuxerBoxes::FBoxNMHD;
			MINF->AddChild(NMHD);
		}
		else if (bIsOpaque)
		{
			auto const& op = TrackSpecs[TrackIdx].Properties.Get<FTrackSpec::FOpaqueData>();
			HDLR->SetHandlerType(op.mdia_hdlr_Type);
			HDLR->SetName(op.mdia_hdlr_Name);
			MP4MuxerBoxes::FBoxOpaqueData* MediaHeader = new MP4MuxerBoxes::FBoxOpaqueData(op.minf_MediaHeader_Type);
			MediaHeader->SetRawBoxData(op.minf_MediaHeader_RawBoxData);
			MINF->AddChild(MediaHeader);
		}
		else
		{
			unimplemented();
			return false;
		}

		// Track name, if any.
		if (!TrackSpecs[TrackIdx].Name.IsEmpty())
		{
			MP4MuxerBoxes::FBoxUDTA* UDTA = new MP4MuxerBoxes::FBoxUDTA;
			// Write the name in a simple box. Most tools expect it that way even though it
			// should be a full box according to Apple (the `name` box is defined by QuickTime)
			MP4MuxerBoxes::FBoxNAMEbase* NAME = new MP4MuxerBoxes::FBoxNAMEbase;
			NAME->SetName(TrackSpecs[TrackIdx].Name);
			TRAK->AddChild(UDTA);
			UDTA->AddChild(NAME);
		}

		// DINF/DREF/URL
		MP4MuxerBoxes::FBoxDINF* DINF = new MP4MuxerBoxes::FBoxDINF;
		MINF->AddChild(DINF);
		MP4MuxerBoxes::FBoxDREF* DREF = new MP4MuxerBoxes::FBoxDREF;
		DINF->AddChild(DREF);
		MP4MuxerBoxes::FBoxURL* URL_ = new MP4MuxerBoxes::FBoxURL;
		DREF->AddChild(URL_);

		// STBL
		MP4MuxerBoxes::FBoxSTBL* STBL = new MP4MuxerBoxes::FBoxSTBL;
		MINF->AddChild(STBL);
		MP4MuxerBoxes::FBoxSTSD* STSD = new MP4MuxerBoxes::FBoxSTSD;
		STBL->AddChild(STSD);
		if (bIsVideo)
		{
			auto const& vid = TrackSpecs[TrackIdx].Properties.Get<FTrackSpec::FVideo>();
			MP4MuxerBoxes::FBoxVisualSampleEntry* Visual = new MP4MuxerBoxes::FBoxVisualSampleEntry(TrackSpecs[TrackIdx].SampleEntryFormat);
			Visual->SetWidth((uint16)vid.DisplayWidth);
			Visual->SetHeight((uint16)vid.DisplayHeight);
			Visual->SetCompressorName(vid.CompressorName);
			Visual->SetAdditionalBoxes(TrackSpecs[TrackIdx].SampleEntryBoxes);
			STSD->AddChild(Visual);
		}
		else if (bIsAudio)
		{
			auto const& aud = TrackSpecs[TrackIdx].Properties.Get<FTrackSpec::FAudio>();
			MP4MuxerBoxes::FBoxAudioSampleEntry* Audio = new MP4MuxerBoxes::FBoxAudioSampleEntry(TrackSpecs[TrackIdx].SampleEntryFormat);
			if (Audio->SetSampleRate(aud.SamplingRate))
			{
				STSD->SetVersion(1);
			}
			Audio->SetChannelCount(aud.NumberOfChannels);
			Audio->SetAdditionalBoxes(TrackSpecs[TrackIdx].SampleEntryBoxes);
			STSD->AddChild(Audio);
		}
		else if (bIsTimecode)
		{
			auto const& tc = TrackSpecs[TrackIdx].Properties.Get<FTrackSpec::FTimecode>();
			MP4MuxerBoxes::FBoxTimecodeSampleEntry* Timecode = new MP4MuxerBoxes::FBoxTimecodeSampleEntry(TrackSpecs[TrackIdx].SampleEntryFormat);
			Timecode->SetTimescale(tc.Timescale);
			Timecode->SetFrameDuration(tc.FrameDuration);
			Timecode->SetFramesPerSecond(tc.FramesPerSecond);
			Timecode->SetDropFrame(tc.bDropFrame);
			Timecode->SetMax24Hours(tc.bMax24Hours);
			Timecode->SetAllowNegativeTimes(tc.bAllowNegativeTimes);
			STSD->AddChild(Timecode);
		}
		else if (bIsOpaque)
		{
			auto const& od = TrackSpecs[TrackIdx].Properties.Get<FTrackSpec::FOpaqueData>();
			MP4MuxerBoxes::FBoxOpaqueData* SampleEntry = new MP4MuxerBoxes::FBoxOpaqueData(TrackSpecs[TrackIdx].SampleEntryFormat);
			SampleEntry->SetRawBoxData(od.stsd_SampleEntry_RawBoxData);
			STSD->AddChild(SampleEntry);
		}
		else
		{
			unimplemented();
			return false;
		}

		MP4MuxerBoxes::FBoxSTTS* STTS = new MP4MuxerBoxes::FBoxSTTS;
		TrackChunkSamples[TrackIdx].TrackMuxBoxes.STTS = STTS;
		STBL->AddChild(STTS);
		if (Configuration.MuxMode != IMP4RawMuxer::FConfiguration::EMuxMode::Fragmented)
		{
			if (!TrackSpecs[TrackIdx].bIsAllKeyframes)
			{
				// Sync sample box (not required when every sample is a sync sample or when writing fragmented file)
				MP4MuxerBoxes::FBoxSTSS* STSS = new MP4MuxerBoxes::FBoxSTSS;
				TrackChunkSamples[TrackIdx].TrackMuxBoxes.STSS = STSS;
				STBL->AddChild(STSS);
			}
			// Composition time offset box (should not be required when every sample is a sync sample though)
			MP4MuxerBoxes::FBoxCTTS* CTTS = new MP4MuxerBoxes::FBoxCTTS;
			TrackChunkSamples[TrackIdx].TrackMuxBoxes.CTTS = CTTS;
			STBL->AddChild(CTTS);
		}
		MP4MuxerBoxes::FBoxSTSC* STSC = new MP4MuxerBoxes::FBoxSTSC;
		TrackChunkSamples[TrackIdx].TrackMuxBoxes.STSC = STSC;
		STBL->AddChild(STSC);
		MP4MuxerBoxes::FBoxSTSZ* STSZ = new MP4MuxerBoxes::FBoxSTSZ;
		TrackChunkSamples[TrackIdx].TrackMuxBoxes.STSZ = STSZ;
		STBL->AddChild(STSZ);
		MP4MuxerBoxes::FBoxSTCO* STCO = new MP4MuxerBoxes::FBoxSTCO;
		TrackChunkSamples[TrackIdx].TrackMuxBoxes.STCO = STCO;
		STBL->AddChild(STCO);
		// Sub-sample information? These go into the `stbl` for standard files and into the `traf` for fragmented files.
		if (Configuration.MuxMode != IMP4RawMuxer::FConfiguration::EMuxMode::Fragmented && TrackSpecs[TrackIdx].SubSampleFlags.IsSet())
		{
			MP4MuxerBoxes::FBoxSUBS* SUBS = new MP4MuxerBoxes::FBoxSUBS;
			TrackChunkSamples[TrackIdx].TrackMuxBoxes.SUBS = SUBS;
			STBL->AddChild(SUBS);
			SUBS->SetFlags(TrackSpecs[TrackIdx].SubSampleFlags.GetValue());
		}
	}
	return true;
}


bool FMP4RawMuxer::WriteUpdate(int64 InFilePosition, TConstArrayView<uint8> InData)
{
	if (!FileHandle)
	{
		return false;
	}
	int64 PosNow = FileHandle->Tell();
	return FileHandle->Seek(InFilePosition) && FileHandle->Write(InData.GetData(), InData.Num()) && FileHandle->Seek(PosNow);
}


bool FMP4RawMuxer::WriteFileEnd()
{
	if (!FTYP || !MOOV || !MVHD)
	{
		return false;
	}
	int64 FilePosAtEnd = FileHandle->Tell();

	// Calculate a common track time scale if possible to use in the movie boxes.
	uint32 MovieTimescale = TrackChunkSamples.Num() ? TrackChunkSamples[0].Timescale : MVHD->GetTimescale();
	uint64 LcmTimescale = MovieTimescale;
	for(int32 i=1,iMax=TrackChunkSamples.Num(); i<iMax; ++i)
	{
		LcmTimescale = std::lcm((uint64)LcmTimescale, (uint64)TrackChunkSamples[i].Timescale);
		// Excessivily large such that we cannot represent a minimum of 7200 seconds without overflowing a 32 bit value?
		if (LcmTimescale > 0x80000)
		{
			MovieTimescale = MVHD->GetTimescale();
			break;
		}
		MovieTimescale = (uint32)LcmTimescale;
	}

	if (MovieTimescale != MVHD->GetTimescale())
	{
		if (!MVHD->UpdateTimescale(MovieTimescale))
		{
			return SetUpdateError(TEXT("timescale"), TEXT("mvhd"));
		}
	}

	// Update the track durations in the `tkhd` and `mdhd` boxes as well as the movie duration in `mvhd` and `mehd`
	int64 LongestTrackInMovieUnits = 0;
	for(auto &tcs : TrackChunkSamples)
	{
		int64 TrackDurationInMovieUnits = MP4Utilities::ConvertToTimescale(MovieTimescale, tcs.TotalTrackDuration, tcs.Timescale);
		if (TrackDurationInMovieUnits > LongestTrackInMovieUnits)
		{
			LongestTrackInMovieUnits = TrackDurationInMovieUnits;
		}
		if (!tcs.TrackMuxBoxes.TKHD->UpdateDuration(TrackDurationInMovieUnits))
		{
			return SetUpdateError(TEXT("duration"), TEXT("tkhd"));
		}
		if (!tcs.TrackMuxBoxes.TKHD->RewriteChanges(RewriteChangeDelegate))
		{
			return SetWriteError();
		}
		if (!tcs.TrackMuxBoxes.MDHD->UpdateDuration(tcs.TotalTrackDuration))
		{
			return SetUpdateError(TEXT("duration"), TEXT("mdhd"));
		}
		if (!tcs.TrackMuxBoxes.MDHD->RewriteChanges(RewriteChangeDelegate))
		{
			return SetWriteError();
		}
		if (tcs.TrackMuxBoxes.ELST)
		{
			if (!tcs.TrackMuxBoxes.ELST->UpdateEditDuration(tcs.TotalTrackDuration))
			{
				return SetUpdateError(TEXT("edit duration"), TEXT("elst"));
			}
			if (!tcs.TrackMuxBoxes.ELST->RewriteChanges(RewriteChangeDelegate))
			{
				return SetWriteError();
			}
		}
	}
	if (!MVHD->UpdateDuration(LongestTrackInMovieUnits))
	{
		return SetUpdateError(TEXT("duration"), TEXT("mvhd"));
	}
	if (!MVHD->RewriteChanges(RewriteChangeDelegate))
	{
		return SetWriteError();
	}
	if (MEHD)
	{
		if (!MEHD->UpdateFragmentDuration(LongestTrackInMovieUnits))
		{
			return SetUpdateError(TEXT("fragment duration"), TEXT("mehd"));
		}
		if (!MEHD->RewriteChanges(RewriteChangeDelegate))
		{
			return SetWriteError();
		}
	}

	// In standard mode the `moov` box has not been built yet. Do it and write it out.
	if (Configuration.MuxMode != IMP4RawMuxer::FConfiguration::EMuxMode::Fragmented)
	{
		// Check to see if we really need to write composition offsets for the tracks.
		for(auto &tcs : TrackChunkSamples)
		{
			if (!tcs.TrackMuxBoxes.CTTS)
			{
				continue;
			}
			// Is it needed?
			if (!tcs.TrackMuxBoxes.CTTS->HaveCompositionOffsets())
			{
				// Not needed, remove it.
				tcs.TrackMuxBoxes.CTTS->GetParent()->RemoveChild(tcs.TrackMuxBoxes.CTTS);
				delete tcs.TrackMuxBoxes.CTTS;
				tcs.TrackMuxBoxes.CTTS = nullptr;
			}
		}

		// Check if any chunk offset box requires 64 bits. If one do then
		// we set all to require 64 bits.
		bool bUsing64BitChunkOffsets = false;
		for(auto &tcs : TrackChunkSamples)
		{
			if (tcs.TrackMuxBoxes.STCO->Needs64Bits())
			{
				bUsing64BitChunkOffsets = true;
				for(auto &tcs64 : TrackChunkSamples)
				{
					tcs64.TrackMuxBoxes.STCO->Force64Bits();
				}
				break;
			}
		}

		// Remember the file position where we will write the `moov` box.
		check(FileHandle->Tell() == FilePosAtEnd);
		int64 FilePosWhereMoovStarts = FilePosAtEnd;

		MP4MuxerBoxes::FBoxBuilderData builderData;
		MOOV->BuildData(builderData);
		MOOV->CalculateBoxSizes();
		int64 MoovSize = MOOV->GetSize();
		// Check if in WebOptimized mode the size of the file (mdat and moov (and others) combined)
		// exceeds 4 GiB. If so we have to use 64 bit chunk offsets then because the `moov` will com
		// before the `mdat` and thus all chunk offsets get larger and may not fit into 32 bits any more.
		if (Configuration.MuxMode == IMP4RawMuxer::FConfiguration::EMuxMode::WebOptimized)
		{
			FileEndOffsetOfStandardMDAT = FilePosAtEnd;

			int64 TotalFileSize = FilePosWhereMoovStarts + MoovSize;
			const int32 kExtraSpace = 65536;
			if (bUsing64BitChunkOffsets == false && (TotalFileSize + kExtraSpace) >= 0xffffffff)
			{
				for(auto &tcs64 : TrackChunkSamples)
				{
					tcs64.TrackMuxBoxes.STCO->Force64Bits();
				}
				// Rebuild the box tree.
				MOOV->ClearBuiltBoxData();
				MOOV->BuildData(builderData);
				MOOV->CalculateBoxSizes();
			}
		}

		bool bWriteOk = MOOV->VisitForWriting([&](MP4MuxerBoxes::FBoxBase* InBox)
		{
			InBox->SetAbsoluteFileOffset(FileHandle->Tell());
			return InBox->GetCompiledBoxData().Num() ? FileHandle->Write(InBox->GetCompiledBoxData().GetData(), InBox->GetCompiledBoxData().Num()) : true;
		});
		if (!bWriteOk)
		{
			return SetWriteError();
		}

		// Get the final size of the `mdat` box.
		const int64 SizeForMDATBox = FilePosAtEnd - FileOffsetOfStandardMDAT;
		TArray<uint8> MDATBox;
		if (SizeForMDATBox > 0xffffffffU)
		{
			FileOffsetOfStandardMDAT -= 8;
			MDATBox.SetNumZeroed(16);
			uint32* bd = (uint32*)MDATBox.GetData();
			*bd++ = MP4MuxerBoxes::EndianSwap((uint32)1U);
			*bd++ = MP4MuxerBoxes::EndianSwap(MP4MuxerBoxes::BoxAtom('m','d','a','t'));
			uint64* bd2 = (uint64*)bd;
			*bd2 = MP4MuxerBoxes::EndianSwap((uint64)(SizeForMDATBox + 8));
		}
		else
		{
			MDATBox.SetNumZeroed(8);
			uint32* bd = (uint32*)MDATBox.GetData();
			*bd++ = MP4MuxerBoxes::EndianSwap((uint32)SizeForMDATBox);
			*bd++ = MP4MuxerBoxes::EndianSwap(MP4MuxerBoxes::BoxAtom('m','d','a','t'));
		}
		bWriteOk = FileHandle->Seek(FileOffsetOfStandardMDAT) && FileHandle->Write(MDATBox.GetData(), MDATBox.Num()) && FileHandle->Seek(FilePosAtEnd);
		if (!bWriteOk)
		{
			return SetWriteError();
		}
	}

	// Need to change the major brand?
	if (MinimumMajorISOBrand != WrittenMajorISOBrand)
	{
		if (MinimumMajorISOBrand <= 9)
		{
			if (!FTYP->UpdateMajorBrand(MP4MuxerBoxes::BoxAtom('i','s','o','0'+MinimumMajorISOBrand)))
			{
				return SetUpdateError(TEXT("major brand"), TEXT("ftyp"));
			}
		}
		else if (MinimumMajorISOBrand <= 11)
		{
			if (!FTYP->UpdateMajorBrand(MP4MuxerBoxes::BoxAtom('i','s','o','a'+MinimumMajorISOBrand-10)))
			{
				return SetUpdateError(TEXT("major brand"), TEXT("ftyp"));
			}
		}
		if (!FTYP->RewriteChanges(RewriteChangeDelegate))
		{
			return SetWriteError();
		}
	}
	return true;
}

bool FMP4RawMuxer::RequestSampleData()
{
	TArray<FSampleRequest> TracksNeedingSamples;
	FScopeLock lock(&Lock);
	for(int32 i=0; i<TrackChunkSamples.Num(); ++i)
	{
		// Is this track still active?
		if (!TrackChunkSamples[i].bHasFinished && !TrackChunkSamples[i].bGotLastSample)
		{
			// Does it need data?
			if (TrackChunkSamples[i].AccumulatedDuration < TrackChunkSamples[i].InterleaveDurationNeeded)
			{
				FSampleRequest& sr(TracksNeedingSamples.Emplace_GetRef());
				sr.TrackIndex = i;
				sr.SampleNumber = TrackChunkSamples[i].NextSampleNumToRequest;
			}
		}
	}
	TracksAwaitingSamples = TracksNeedingSamples;
	lock.Unlock();
	SampleRequestDelegate.ExecuteIfBound(TracksNeedingSamples);
	return TracksNeedingSamples.IsEmpty();
}

bool FMP4RawMuxer::AddTrackSample(int32 InTrack, const FTrackSample& InTrackSample)
{
	if (bFailed || LastError.Len())
	{
		return false;
	}
	FScopeLock lock(&Lock);
	int32 srIdx = TracksAwaitingSamples.IndexOfByPredicate([&](const FSampleRequest& r){ return r.TrackIndex == InTrack; });
	if (srIdx == INDEX_NONE)
	{
		return SetError(FString::Printf(TEXT("Called to provide sample data for track %d which has not requested sample data."), InTrack));
	}
	FTrackChunkSamples& tcs(TrackChunkSamples[InTrack]);
	if (InTrackSample.SampleNumber != tcs.NextSampleNumToRequest && !InTrackSample.bIsFinalSample)
	{
		return SetError(FString::Printf(TEXT("Called to provide sample data for track %d sample number %u, but requested sample number %u."), InTrack, InTrackSample.SampleNumber, tcs.NextSampleNumToRequest));
	}
	TracksAwaitingSamples.RemoveAt(srIdx);
	if (InTrackSample.Data.Num())
	{
		uint32 Duration = InTrackSample.Duration ? InTrackSample.Duration : TrackSpecs[InTrack].ConstantSampleDuration;
		const bool bIsKeyframeOnly = TrackSpecs[InTrack].bIsAllKeyframes;
		bool bIsKeyframe = bIsKeyframeOnly ? true : InTrackSample.bIsKeyframe;
		/*
			Zero duration _may_ be set for the very last sample only. We don't really know if this is the last sample
			(unless `InTrackSample.bIsFinalSample` is set, which is does not have to be) so we cannot reliably detect
			this to be valid or not.

			if (Duration == 0)
			{
				return SetError(FString::Printf(TEXT("Sample duration must not be zero or less")));
			}
		*/
		// There is no version of a `stsz`, `stz2` or `trun` that would allow for a single AU to be larger than 4GiB (32 bits)
		if (InTrackSample.Data.NumBytes() > 0xffffffffU)
		{
			return SetError(FString::Printf(TEXT("An ISO/IEC 14496-12 file can not store access units exceeding 4 GiB.")));
		}
		if (InTrackSample.SubSamples.Num() && !TrackSpecs[InTrack].SubSampleFlags.IsSet())
		{
			return SetError(FString::Printf(TEXT("Sample provides subsample information but the track was not configured for subsamples.")));
		}

		tcs.TotalTrackDuration += Duration;
		tcs.AccumulatedDuration += Duration;
		tcs.SumOfSampleDurations += Duration;

		TUniquePtr<FTrackChunkSamples::FSample> ns = MakeUnique<FTrackChunkSamples::FSample>();
		ns->SubSamples = InTrackSample.SubSamples;
		ns->Data = InTrackSample.Data;
		ns->DTS = InTrackSample.DTS;
		ns->PTS = bIsKeyframeOnly ? InTrackSample.DTS : InTrackSample.PTS;
		ns->Duration = Duration;
		ns->bIsKeyframe = bIsKeyframe;
		tcs.Samples.Emplace(MoveTemp(ns));
		++tcs.NextSampleNumToRequest;
	}
	tcs.bGotLastSample = InTrackSample.bIsFinalSample;
	lock.Unlock();
	if (WorkerEvent)
	{
		WorkerEvent->Trigger();
	}
	return true;
}


bool FMP4RawMuxer::WriteAccumulatedSamples()
{
	int32 NumActiveTracks = 0;
	for(int32 i=0; i<TrackChunkSamples.Num(); ++i)
	{
		NumActiveTracks += !TrackChunkSamples[i].bHasFinished ? 1 : 0;
	}
	if (NumActiveTracks == 0)
	{
		return true;
	}
	// Write standard or fragmented.
	bool bOk = Configuration.MuxMode != IMP4RawMuxer::FConfiguration::EMuxMode::Fragmented ? WriteAccumulatedSamplesStandard() : WriteAccumulatedSamplesFragmented();
	// Dump the samples that were written.
	for(auto &tcs : TrackChunkSamples)
	{
		tcs.Empty();
		if (tcs.bGotLastSample)
		{
			tcs.bHasFinished = true;
		}
	}
	// Continue with next batch of samples.
	WorkerEvent->Trigger();
	return bOk;
}

bool FMP4RawMuxer::WriteAccumulatedSamplesStandard()
{
	for(int32 nTrk=0; nTrk<TrackChunkSamples.Num(); ++nTrk)
	{
		const FTrackChunkSamples& tcs(TrackChunkSamples[nTrk]);
		// Skip if track no longer active.
		if (tcs.bHasFinished || tcs.Samples.IsEmpty())
		{
			continue;
		}
		if (!tcs.TrackMuxBoxes.STSZ || !tcs.TrackMuxBoxes.STTS || !tcs.TrackMuxBoxes.CTTS || !tcs.TrackMuxBoxes.STSC || !tcs.TrackMuxBoxes.STCO)
		{
			return SetError(TEXT("Internal error, one of the required boxes has not been created yet."));
		}

		// Where in the file are we at now?
		int64 FileDataOffset = FileHandle->Tell();
		// Write the samples of this chunk.
		for(int32 i=0; i<tcs.Samples.Num(); ++i)
		{
			const FTrackChunkSamples::FSample* const Smp = tcs.Samples[i].Get();
			bool bWriteOk = FileHandle->Write(Smp->Data.GetData(), Smp->Data.Num());
			if (!bWriteOk)
			{
				return SetWriteError();
			}
			// Set the sample size. Do this first since that allows us to see how many samples there are in total.
			tcs.TrackMuxBoxes.STSZ->AddSampleSize((uint32) Smp->Data.Num());
			// Set the sample duration.
			tcs.TrackMuxBoxes.STTS->AddSampleDuration(Smp->Duration);
			// Sync sample is optional to write.
			if (Smp->bIsKeyframe && tcs.TrackMuxBoxes.STSS)
			{
				tcs.TrackMuxBoxes.STSS->AddSampleIndex(tcs.TrackMuxBoxes.STSZ->GetNumberOfSamples());
			}

			// On the very first sample, if it has a negative DTS we might want to add an edit list.
			if (Smp->DTS < 0 && tcs.TrackMuxBoxes.STSZ->GetNumberOfSamples() == 1)
			{
				MP4MuxerBoxes::FBoxEDTS* EDTS = new MP4MuxerBoxes::FBoxEDTS;
				tcs.TrackMuxBoxes.TKHD->GetParent()->AddChildAfter(EDTS, tcs.TrackMuxBoxes.TKHD);
				MP4MuxerBoxes::FBoxELST* ELST = new MP4MuxerBoxes::FBoxELST;
				TrackChunkSamples[nTrk].TrackMuxBoxes.ELST = ELST;
				EDTS->AddChild(ELST);
				ELST->SetMediaTime(-Smp->DTS);
			}

			// Set composition time offset.
			int32 CompositionTimeOffset = Smp->PTS - Smp->DTS;
			tcs.TrackMuxBoxes.CTTS->AddSampleCompositionOffset(CompositionTimeOffset);

			// Subsamples?
			if (Smp->SubSamples.Num() && tcs.TrackMuxBoxes.SUBS)
			{
				tcs.TrackMuxBoxes.SUBS->AddSubsamplesForSample(tcs.TrackMuxBoxes.STSZ->GetNumberOfSamples(), Smp->SubSamples);
			}
		}
		// Negative composition offsets require the major brand to be at least 4.
		if (MinimumMajorISOBrand < 4 && tcs.TrackMuxBoxes.CTTS->HasNegativeCompositionOffsets() && tcs.TrackMuxBoxes.CTTS->HaveCompositionOffsets())
		{
			MinimumMajorISOBrand = 4;
		}
		// Add chunk offset.
		tcs.TrackMuxBoxes.STCO->AddChunkOffset(FileDataOffset);
		// Add sample to chunk entry.
		tcs.TrackMuxBoxes.STSC->AddEntry(tcs.TrackMuxBoxes.STCO->GetNumberOfChunkOffsets(), (uint32)tcs.Samples.Num(), 1U);
	}
	return true;
}

bool FMP4RawMuxer::WriteAccumulatedSamplesFragmented()
{
	// On the very first sample, if it has a negative DTS we might want to add an edit list.
	if (FragmentSequenceNumber == 0)
	{
		for(int32 nTrk=0; nTrk<TrackChunkSamples.Num(); ++nTrk)
		{
			const FTrackChunkSamples& tcs(TrackChunkSamples[nTrk]);
			if (tcs.Samples.Num() && tcs.Samples[0]->DTS < 0 && !tcs.TrackMuxBoxes.ELST)
			{
				MP4MuxerBoxes::FBoxEDTS* EDTS = new MP4MuxerBoxes::FBoxEDTS;
				tcs.TrackMuxBoxes.TKHD->GetParent()->AddChildAfter(EDTS, tcs.TrackMuxBoxes.TKHD);

				MP4MuxerBoxes::FBoxELST* ELST = new MP4MuxerBoxes::FBoxELST;
				TrackChunkSamples[nTrk].TrackMuxBoxes.ELST = ELST;
				EDTS->AddChild(ELST);
				ELST->SetMediaTime(-tcs.Samples[0]->DTS);
			}
		}
		// Write out the `moov` box now.
		if (!bWroteMOOV)
		{
			bWroteMOOV = true;
			MP4MuxerBoxes::FBoxBuilderData builderData;
			MOOV->BuildData(builderData);
			MOOV->CalculateBoxSizes();
			bool bWriteOk = MOOV->VisitForWriting([&](MP4MuxerBoxes::FBoxBase* InBox)
			{
				InBox->SetAbsoluteFileOffset(FileHandle->Tell());
				return InBox->GetCompiledBoxData().Num() ? FileHandle->Write(InBox->GetCompiledBoxData().GetData(), InBox->GetCompiledBoxData().Num()) : true;
			});
			if (!bWriteOk)
			{
				return SetWriteError();
			}
		}
	}

	TUniquePtr<MP4MuxerBoxes::FBoxMOOF> MOOF;
	struct FOffsetPatch
	{
		MP4MuxerBoxes::FBoxTFHD* TFHDBox = nullptr;
		int64 FileOffset = -1;
	};
	TArray<FOffsetPatch> TFHDBoxesToPatch;
	int64 SizeOfAllSamples = 0;
	for(int32 nTrk=0; nTrk<TrackChunkSamples.Num(); ++nTrk)
	{
		const FTrackChunkSamples& tcs(TrackChunkSamples[nTrk]);
		// Skip if track no longer active or has no samples
		if (tcs.bHasFinished || tcs.Samples.IsEmpty())
		{
			continue;
		}
		if (!MOOF)
		{
			MOOF = MakeUnique<MP4MuxerBoxes::FBoxMOOF>();
			MP4MuxerBoxes::FBoxMFHD* MFHD = new MP4MuxerBoxes::FBoxMFHD;
			MOOF->AddChild(MFHD);
			MFHD->SetSequenceNumber(++FragmentSequenceNumber);
		}

		MP4MuxerBoxes::FBoxTRAF* TRAF = new MP4MuxerBoxes::FBoxTRAF;
		MOOF->AddChild(TRAF);
		MP4MuxerBoxes::FBoxTFHD* TFHD = new MP4MuxerBoxes::FBoxTFHD;
		TRAF->AddChild(TFHD);
		MP4MuxerBoxes::FBoxTFDT* TFDT = new MP4MuxerBoxes::FBoxTFDT;
		TRAF->AddChild(TFDT);
		TFDT->SetBaseMediaDecodeTime((uint64)(tcs.TotalTrackDuration - tcs.SumOfSampleDurations));
		TFHD->SetTrackID(1U + nTrk);
		TFHD->SetBaseDataOffset(0);
		/*
			If using default-base-is-moof the major brand needs to be at least 5.

			if (MinimumMajorISOBrand < 5)
			{
				MinimumMajorISOBrand = 5;
			}
		*/
		TFHDBoxesToPatch.Add({TFHD});

		const bool bAllAreSyncSamples = TrackSpecs[nTrk].bIsAllKeyframes;
		const uint32 DefaultSampleFlags = tcs.TrackMuxBoxes.TREX ? tcs.TrackMuxBoxes.TREX->GetDefaultSampleFlags() : 0U;

		MP4MuxerBoxes::FBoxTRUN* TRUN = new MP4MuxerBoxes::FBoxTRUN;
		TRAF->AddChild(TRUN);
		TRUN->SetDefaultSampleFlagsForTesting(DefaultSampleFlags);

		// Sub-sample information
		MP4MuxerBoxes::FBoxSUBS* SUBS = nullptr;

		// Process samples.
		for(int32 i=0; i<tcs.Samples.Num(); ++i)
		{
			const FTrackChunkSamples::FSample* const Smp = tcs.Samples[i].Get();
			uint32 Flags;
			if (bAllAreSyncSamples || Smp->bIsKeyframe)
			{
				Flags = DefaultSampleFlags & ~0x10000U;
			}
			else
			{
				Flags = DefaultSampleFlags | 0x10000U;
			}

			int32 CompositionTimeOffset = Smp->PTS - Smp->DTS;
			// Negative composition offsets in the `trun` require the major brand to be at least 6.
			if (CompositionTimeOffset < 0 && MinimumMajorISOBrand < 6)
			{
				MinimumMajorISOBrand = 6;
			}
			TRUN->AddSample(Smp->Duration, Smp->Data.Num(), Flags, CompositionTimeOffset);
			// Accumulate the size of all samples, which is how big the following `mdat` box
			// is going to be. This could exceed 4 GiB which may require a special size indicator
			// which is difficult to patch in later, so it's best if we know upfront if we need it.
			SizeOfAllSamples += Smp->Data.Num();

			// Subsamples?
			if (Smp->SubSamples.Num() && TrackSpecs[nTrk].SubSampleFlags.IsSet())
			{
				if (!SUBS)
				{
					SUBS = new MP4MuxerBoxes::FBoxSUBS;
					TRAF->AddChild(SUBS);
					SUBS->SetFlags(TrackSpecs[nTrk].SubSampleFlags.GetValue());
				}
				SUBS->AddSubsamplesForSample(i + 1, Smp->SubSamples);
			}
		}
		// If there is no constant sample duration given for the track in general but the
		// samples in this run all have the same duration, set the constant duration for this run.
		if (!TRUN->HasDifferentDurations())
		{
			// If there is a constant sample duration for the track and it's the same
			// as the constant duration of the samples in this run then we do not need
			// to write per-sample durations.
			if (TrackSpecs[nTrk].ConstantSampleDuration == tcs.Samples[0]->Duration)
			{
				TRUN->DoNotWriteDurations();
			}
			// If there is no constant sample duration set for the track then put this
			// run's constant duration into the track fragment header.
			else if (TrackSpecs[nTrk].ConstantSampleDuration == 0)
			{
				TFHD->SetDefaultSampleDuration(tcs.Samples[0]->Duration);
				TRUN->DoNotWriteDurations();
			}
		}

		// Same for the sizes.
		// TODO: check that if fixed size is given it matches the sample size!
		if (/*TrackSpecs[nTrk].ConstantSampleSize == 0 &&*/ !TRUN->HasDifferentSizes())
		{
			TFHD->SetDefaultSampleSize((uint32) tcs.Samples[0]->Data.Num());
			TRUN->DoNotWriteSizes();
		}
		// When sample flags differ during the run then the flags need to be written and we can skip
		// any special handling.
		if (!TRUN->AreFlagsAfterFirstSampleDifferent())
		{
			// If only the first sample has flags other than the default while the rest uses default, set first sample flags only.
			if (TRUN->GetSubsequentSampleFlagsOrDefault() == DefaultSampleFlags && TRUN->GetFirstSampleFlagsOrDefault() != DefaultSampleFlags)
			{
				TRUN->SetFirstSampleFlags(TRUN->GetFirstSampleFlagsOrDefault());
			}
			// Are all samples using default value?
			else if (TRUN->GetFirstSampleFlagsOrDefault() == TRUN->GetSubsequentSampleFlagsOrDefault() && TRUN->GetFirstSampleFlagsOrDefault() == DefaultSampleFlags)
			{
				TRUN->DoNotWriteFlags();
			}
			else
			{
				// The samples after the first all have the same flags. Set that in the trak header box.
				TFHD->SetDefaultSampleFlags(TRUN->GetSubsequentSampleFlagsOrDefault());
				// If the first sample has a different value set it up.
				if (TRUN->GetFirstSampleFlagsOrDefault() != TRUN->GetSubsequentSampleFlagsOrDefault())
				{
					TRUN->SetFirstSampleFlags(TRUN->GetFirstSampleFlagsOrDefault());
				}
				TRUN->DoNotWriteFlags();
			}
		}
	}
	if (!MOOF)
	{
		return true;
	}
	MP4MuxerBoxes::FBoxBuilderData builderData;
	MOOF->BuildData(builderData);
	MOOF->CalculateBoxSizes();
	bool bWriteOk = MOOF->VisitForWriting([&](MP4MuxerBoxes::FBoxBase* InBox)
	{
		InBox->SetAbsoluteFileOffset(FileHandle->Tell());
		return InBox->GetCompiledBoxData().Num() ? FileHandle->Write(InBox->GetCompiledBoxData().GetData(), InBox->GetCompiledBoxData().Num()) : true;
	});
	if (!bWriteOk)
	{
		return SetWriteError();
	}

	// Write the `mdat` box.
	int64 SizeForMDATBox = 8 + SizeOfAllSamples;
	TArray<uint8> MDATBox;
	if (SizeForMDATBox > 0xffffffffU)
	{
		MDATBox.SetNumZeroed(16);
		uint32* bd = (uint32*)MDATBox.GetData();
		*bd++ = MP4MuxerBoxes::EndianSwap((uint32)1U);
		*bd++ = MP4MuxerBoxes::EndianSwap(MP4MuxerBoxes::BoxAtom('m','d','a','t'));
		uint64* bd2 = (uint64*)bd;
		*bd2 = MP4MuxerBoxes::EndianSwap((uint64)(SizeForMDATBox + 8));
	}
	else
	{
		MDATBox.SetNumZeroed(8);
		uint32* bd = (uint32*)MDATBox.GetData();
		*bd++ = MP4MuxerBoxes::EndianSwap((uint32)SizeForMDATBox);
		*bd++ = MP4MuxerBoxes::EndianSwap(MP4MuxerBoxes::BoxAtom('m','d','a','t'));
	}
	bWriteOk = FileHandle->Write(MDATBox.GetData(), MDATBox.Num());
	if (!bWriteOk)
	{
		return SetWriteError();
	}

	// Write the samples
	int64 FileDataOffset = FileHandle->Tell();
	int32 PatchNum = 0;
	for(int32 nTrk=0; nTrk<TrackChunkSamples.Num(); ++nTrk)
	{
		const FTrackChunkSamples& tcs(TrackChunkSamples[nTrk]);
		// Skip if track no longer active or has no samples
		if (tcs.bHasFinished || tcs.Samples.IsEmpty())
		{
			continue;
		}
		TFHDBoxesToPatch[PatchNum++].FileOffset = FileHandle->Tell();
		for(int32 i=0; i<tcs.Samples.Num(); ++i)
		{
			const FTrackChunkSamples::FSample* const Smp = tcs.Samples[i].Get();
			bWriteOk = FileHandle->Write(Smp->Data.GetData(), Smp->Data.Num());
			if (!bWriteOk)
			{
				return SetWriteError();
			}
		}
	}
	// Patch the offset in the `tfhd` boxes
	for(int32 i=0; i<TFHDBoxesToPatch.Num(); ++i)
	{
		if (!TFHDBoxesToPatch[i].TFHDBox->UpdateBaseDataOffset((uint64)TFHDBoxesToPatch[i].FileOffset))
		{
			return SetUpdateError(TEXT("base data offset"), TEXT("tfhd"));
		}
		bWriteOk = TFHDBoxesToPatch[i].TFHDBox->RewriteChanges(RewriteChangeDelegate);
		if (!bWriteOk)
		{
			return SetWriteError();
		}
	}
	return true;
}


bool FMP4RawMuxer::RewriteWebOptimizedFile(bool bInForceComplete)
{
	FilenameWritingNow = Configuration.OutputFilename;
	FileHandle = FPlatformFileManager::Get().GetPlatformFile().OpenWrite(*FilenameWritingNow, false, false);
	if (!FileHandle)
	{
		return SetError(FString::Printf(TEXT("Could not open file \"%s\" for writing."), *FilenameWritingNow));
	}

	// Write the `ftyp` box
	bool bWriteOk = FTYP->VisitForWriting([&](MP4MuxerBoxes::FBoxBase* InBox)
	{
		InBox->SetAbsoluteFileOffset(FileHandle->Tell());
		return InBox->GetCompiledBoxData().Num() ? FileHandle->Write(InBox->GetCompiledBoxData().GetData(), InBox->GetCompiledBoxData().Num()) : true;
	});
	if (!bWriteOk)
	{
		return SetWriteError();
	}

	// Write the `moov` box
	int64 MoovBoxSizeNow = MOOV->GetSize();
	int64 MoovBoxFileOffset = FileHandle->Tell();
	int64 MoovBoxFileEndOffset = MoovBoxFileOffset + MOOV->GetSize();

	// Update all file chunk offsets
	int64 OffsetToAdd = MoovBoxFileEndOffset - FileOffsetOfStandardMDAT;
	for(auto &tcs64 : TrackChunkSamples)
	{
		if (!tcs64.TrackMuxBoxes.STCO->AddFileOffset(OffsetToAdd))
		{
			return SetError(FString::Printf(TEXT("Invalid chunk offset while optimizing file \"%s\"."), *FilenameWritingNow));
		}
	}
	// Rebuild the box tree.
	MP4MuxerBoxes::FBoxBuilderData builderData;
	MOOV->ClearBuiltBoxData();
	MOOV->BuildData(builderData);
	MOOV->CalculateBoxSizes();
	int64 NewMoovBoxSize = MOOV->GetSize();
	if (NewMoovBoxSize != MoovBoxSizeNow)
	{
		return SetError(FString::Printf(TEXT("Size of `moov` box changed while optimizing file \"%s\"."), *FilenameWritingNow));
	}
	bWriteOk = MOOV->VisitForWriting([&](MP4MuxerBoxes::FBoxBase* InBox)
	{
		InBox->SetAbsoluteFileOffset(FileHandle->Tell());
		return InBox->GetCompiledBoxData().Num() ? FileHandle->Write(InBox->GetCompiledBoxData().GetData(), InBox->GetCompiledBoxData().Num()) : true;
	});
	if (!bWriteOk)
	{
		return SetWriteError();
	}

	// Copy the `mdat`
	uint32 kTempBufferSize = Configuration.TemporaryOptimizationBufferSize ? Configuration.TemporaryOptimizationBufferSize : 32 * 1024 * 1024;
	uint8* TempBuffer = reinterpret_cast<uint8*>(FMemory::Malloc(kTempBufferSize));
	if (!TempBuffer)
	{
		return SetError(FString::Printf(TEXT("Could not allocate temporary optimization buffer.")));
	}

	IFileHandle* SourceFileHandle = FPlatformFileManager::Get().GetPlatformFile().OpenRead(*Configuration.TemporaryFilename);
	if (!SourceFileHandle)
	{
		FMemory::Free(TempBuffer);
		return SetError(FString::Printf(TEXT("Failed to open temporary file \"%s\" for reading."), *Configuration.TemporaryFilename));
	}
	SourceFileHandle->Seek(FileOffsetOfStandardMDAT);
	int64 SizeToRewrite = FileEndOffsetOfStandardMDAT - FileOffsetOfStandardMDAT;
	int64 SizeToCopy = SizeToRewrite;
	while(SizeToCopy > 0 && (bInForceComplete || !bTerminate))
	{
		int64 Percentage = SizeToRewrite ? (10000 * (SizeToRewrite - SizeToCopy)) / SizeToRewrite : 0;
		Configuration.OptimizeProgressDelegate.ExecuteIfBound((float)Percentage / 100.0);

		int64 NumToRead = SizeToCopy > kTempBufferSize ? kTempBufferSize : SizeToCopy;
		if (!SourceFileHandle->Read(TempBuffer, NumToRead))
		{
			delete SourceFileHandle;
			FMemory::Free(TempBuffer);
			return SetError(FString::Printf(TEXT("Failed to read from temporary file \"%s\" while optimizing."), *Configuration.TemporaryFilename));
		}
		if (!FileHandle->Write(TempBuffer, NumToRead))
		{
			delete SourceFileHandle;
			FMemory::Free(TempBuffer);
			return SetError(FString::Printf(TEXT("Failed to write to file \"%s\" while optimizing."), *FilenameWritingNow));
		}
		SizeToCopy -= NumToRead;
	}
	delete SourceFileHandle;
	FMemory::Free(TempBuffer);
	if (bInForceComplete || !bTerminate)
	{
		Configuration.OptimizeProgressDelegate.ExecuteIfBound((float)100.0f);
		// Delete the temp file.
		FPlatformFileManager::Get().GetPlatformFile().DeleteFile(*Configuration.TemporaryFilename);
	}
	return true;
}

void FMP4RawMuxer::ClearUpdateBoxes()
{
	// Drop the boxes we needed for updating.
	MOOV.Reset();
	FTYP.Reset();
	MVHD = nullptr;
	MEHD = nullptr;
}

uint32 FMP4RawMuxer::Run()
{
	// Open the file. Temporary one for WebOptimized mode, final output for regular or fragmented mode.
	FilenameWritingNow = Configuration.MuxMode == FConfiguration::EMuxMode::WebOptimized ? Configuration.TemporaryFilename : Configuration.OutputFilename;
	FileHandle = FPlatformFileManager::Get().GetPlatformFile().OpenWrite(*FilenameWritingNow, false, false);
	if (!FileHandle)
	{
		SetError(FString::Printf(TEXT("Could not open file \"%s\" for writing."), *FilenameWritingNow));
		StatusDelegate.ExecuteIfBound(EStatus::Failed);
	}
	else if (!WriteFileStart())
	{
		SetWriteError();
		StatusDelegate.ExecuteIfBound(EStatus::Failed);
	}

	const FTimespan WaitTime(FTimespan::FromMilliseconds(100));
	WorkerEvent->Trigger();
	bool bIsDone = false;
	while(!bTerminate)
	{
		bool bGot = WorkerEvent->Wait(WaitTime);
		if (!bGot)
		{
			continue;
		}
		WorkerEvent->Reset();
		if (bFailed || bIsDone)
		{
			continue;
		}
		// Continue requesting samples until we have enough data for interleaving.
		bool bGotAllSamples = RequestSampleData();
		if (bGotAllSamples)
		{
			// Write them
			if (!WriteAccumulatedSamples())
			{
				SetWriteError();
				StatusDelegate.ExecuteIfBound(EStatus::Failed);
				continue;
			}
			// Are we done?
			bool bAllDone = true;
			for(auto &tcs : TrackChunkSamples)
			{
				if (!tcs.bHasFinished)
				{
					bAllDone = false;
					break;
				}
			}
			if (bAllDone)
			{
				bool bOk = WriteFileEnd();
				CloseFile();
				if (bOk && Configuration.MuxMode == FConfiguration::EMuxMode::WebOptimized)
				{
					// Force completion: if a terminate races in mid-rewrite, finish the file rather
					// than leaving a partial ftyp+moov+truncated-mdat OutputFilename behind.
					bOk = RewriteWebOptimizedFile(/*bInForceComplete*/ true);
					CloseFile();
				}
				ClearUpdateBoxes();
				bIsDone = true;
				if (!bTerminate)
				{
					if (bOk)
					{
						StatusDelegate.ExecuteIfBound(EStatus::Finished);
					}
					else
					{
						StatusDelegate.ExecuteIfBound(EStatus::Failed);
					}
				}
			}
		}
	}

	// Best-effort finalize when terminated mid-stream: write moov and patch the
	// mdat header for whatever samples were already written, so the output isn't
	// left with a placeholder `mdat size=8` and no `moov`.
	if (!bIsDone && !bFailed && FileHandle && FTYP.IsValid() && MOOV.IsValid())
	{
		// Skip if no samples ever landed (terminate raced between WriteFileStart and the
		// first AddTrackSample); finalizing would produce a zero-sample MP4 that isn't useful.
		bool bHaveAnySamples = false;
		for (const FTrackChunkSamples& tcs : TrackChunkSamples)
		{
			if (!tcs.Samples.IsEmpty() || (tcs.TrackMuxBoxes.STSZ && tcs.TrackMuxBoxes.STSZ->GetNumberOfSamples() > 0))
			{
				bHaveAnySamples = true;
				break;
			}
		}
		if (bHaveAnySamples)
		{
			UE_LOGF(LogMP4Muxer, Warning, "FMP4RawMuxer: Worker terminated before completion; attempting best-effort finalize of \"%ls\".", *FilenameWritingNow);
			for (FTrackChunkSamples& tcs : TrackChunkSamples)
			{
				tcs.bGotLastSample = true;
			}
			bool bOk = WriteAccumulatedSamples() && WriteFileEnd();
			CloseFile();
			if (bOk && Configuration.MuxMode == FConfiguration::EMuxMode::WebOptimized)
			{
				// Force completion: bTerminate is already true on this path, and we want the
				// rewrite to actually produce Configuration.OutputFilename rather than bail early.
				bOk = RewriteWebOptimizedFile(/*bInForceComplete*/ true);
				CloseFile();
			}
			ClearUpdateBoxes();
			if (bOk)
			{
				UE_LOGF(LogMP4Muxer, Warning, "FMP4RawMuxer: Best-effort finalize succeeded; output may contain fewer samples than intended.");
				StatusDelegate.ExecuteIfBound(EStatus::Finished);
			}
			else
			{
				UE_LOGF(LogMP4Muxer, Error, "FMP4RawMuxer: Best-effort finalize failed: %ls", *LastError);
				StatusDelegate.ExecuteIfBound(EStatus::Failed);
			}
			bIsDone = true;
		}
		else
		{
			UE_LOGF(LogMP4Muxer, Log, "FMP4RawMuxer: Worker terminated before any samples were written; nothing to finalize for \"%ls\".", *FilenameWritingNow);
		}
	}

	CloseFile();
	return 0;
}
