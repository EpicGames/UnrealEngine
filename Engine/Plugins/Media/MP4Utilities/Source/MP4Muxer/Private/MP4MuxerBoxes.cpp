// Copyright Epic Games, Inc. All Rights Reserved.

#include "MP4MuxerBoxes.h"
#include "Math/NumericLimits.h"
#include "Containers/StringConv.h"

namespace MP4MuxerBoxes
{

namespace
{

class FBoxDataWriter
{
public:
	FBoxDataWriter(TArray<uint8>& InOutBuffer)
		: Buffer(InOutBuffer)
	{
		BytePosition = Buffer.Num();
	}
	~FBoxDataWriter()
	{
		Close();
	}
	void Close()
	{
		AlignToBytes(0);
		Buffer.SetNum(BytePosition, EAllowShrinking::No);
	}

	uint32 PutAlignedValue(uint8 InValue)
	{
		check(IsByteAligned());
		PutByte(InValue);
		return BytePosition - 1;
	}
	uint32 PutAlignedValue(uint16 InValue)
	{
		PutAlignedValue((uint8)(InValue >> 8));
		PutAlignedValue((uint8)(InValue & 0xff));
		return BytePosition - 2;
	}
	uint32 PutAlignedValue(uint32 InValue)
	{
		PutAlignedValue((uint16)(InValue >> 16));
		PutAlignedValue((uint16)(InValue & 0xffff));
		return BytePosition - 4;
	}
	uint32 PutAlignedValue(uint64 InValue)
	{
		PutAlignedValue((uint32)(InValue >> 32));
		PutAlignedValue((uint32)(InValue & 0xffffffff));
		return BytePosition - 8;
	}
	uint32 PutAlignedValue(int8 InValue)
	{
		return PutAlignedValue((uint8)InValue);
	}
	uint32 PutAlignedValue(int16 InValue)
	{
		return PutAlignedValue((uint16)InValue);
	}
	uint32 PutAlignedValue(int32 InValue)
	{
		return PutAlignedValue((uint32)InValue);
	}
	uint32 PutAlignedValue(int64 InValue)
	{
		return PutAlignedValue((uint64)InValue);
	}
	uint32 PutAlignedArray(TConstArrayView<const uint8> InArray)
	{
		for(int32 i=0, iMax=InArray.Num(); i<iMax; ++i)
		{
			PutAlignedValue(InArray[i]);
		}
		return BytePosition - InArray.Num();
	}

	void SetAlignedValueAt(uint8 InValue, uint32 InAtOffset)
	{
		check(InAtOffset < BytePosition);
		Buffer[InAtOffset] = InValue;
	}
	void SetAlignedValueAt(uint16 InValue, uint32 InAtOffset)
	{
		SetAlignedValueAt((uint8)(InValue >> 8), InAtOffset);
		SetAlignedValueAt((uint8)(InValue & 0xff), InAtOffset + 1);
	}
	void SetAlignedValueAt(uint32 InValue, uint32 InAtOffset)
	{
		SetAlignedValueAt((uint16)(InValue >> 16), InAtOffset);
		SetAlignedValueAt((uint16)(InValue & 0xffff), InAtOffset + 2);
	}
	void SetAlignedValueAt(uint64 InValue, uint32 InAtOffset)
	{
		SetAlignedValueAt((uint32)(InValue >> 32), InAtOffset);
		SetAlignedValueAt((uint32)(InValue & 0xffffffff), InAtOffset + 4);
	}

	void SetAlignedValueAt(int8 InValue, uint32 InAtOffset)
	{
		SetAlignedValueAt((uint8)InValue, InAtOffset);
	}
	void SetAlignedValueAt(int16 InValue, uint32 InAtOffset)
	{
		SetAlignedValueAt((uint16)InValue, InAtOffset);
	}
	void SetAlignedValueAt(int32 InValue, uint32 InAtOffset)
	{
		SetAlignedValueAt((uint32)InValue, InAtOffset);
	}
	void SetAlignedValueAt(int64 InValue, uint32 InAtOffset)
	{
		SetAlignedValueAt((uint64)InValue, InAtOffset);
	}


	void PutBits(uint32 InValue, uint32 InNumBits)
	{
		check(InNumBits != 0);
		check(InNumBits <= 32);
		uint32 Mask = 1U << (InNumBits-1);
		while(Mask)
		{
			PutBit((InValue & Mask) != 0 ? 1 : 0);
			Mask >>= 1;
		}
	}
	void PutBits64(uint64 InValue, uint32 InNumBits)
	{
		uint32 Upper = static_cast<uint32>(InValue >> 32U);
		uint32 Lower = static_cast<uint32>(InValue);
		if (InNumBits > 32)
		{
			const uint32 nb = InNumBits - 32;
			PutBits(Upper, nb);
			InNumBits -= nb;
		}
		PutBits(Lower, InNumBits);
	}
	void AlignToBytes(uint32 InFillBitsWith)
	{
		if (BitPosition)
		{
			do
			{
				PutBit(InFillBitsWith);
			} while(BitPosition);
		}
	}
	uint32 GetNumBits() const
	{
		return BytePosition * 8 + BitPosition;
	}
	uint32 GetNumBytesUsed() const
	{
		return (GetNumBits() + 7) / 8;
	}
	bool IsByteAligned() const
	{
		return BitPosition == 0;
	}
	uint32 GetBytePosition() const
	{
		return BytePosition;
	}
private:
	void PutByte(uint8 InValue)
	{
		if (BytePosition == (uint32)Buffer.Num())
		{
			Buffer.SetNumZeroed(Buffer.Num() + 256);
		}
		Buffer[BytePosition++] = InValue;
	}
	void PutBit(uint32 InBit)
	{
		InBit <<= (7 - BitPosition);
		check(Buffer.Num());
		Buffer[BytePosition] |= (uint8)InBit;
		if (++BitPosition == 8)
		{
			BitPosition = 0;
			if (++BytePosition > (uint32)Buffer.Num())
			{
				Buffer.SetNumZeroed(Buffer.Num() + 256);
			}
		}
	}
	TArray<uint8>& Buffer;
	uint32 BytePosition = 0;
	uint32 BitPosition = 0;
};


}







FBoxBase::~FBoxBase()
{
	for(int32 i=0; i<Children.Num(); ++i)
	{
		delete Children[i];
	}
}


void FBoxBase::ClearBuiltBoxData()
{
	for(int32 i=0; i<Children.Num(); ++i)
	{
		Children[i]->ClearBuiltBoxData();
	}
	Size = 0;
	CompiledBoxData.Empty();
	CompiledBoxData_Offset_Size = 0;
	AbsoluteFileOffset = -1;
	bWasBuilt = false;
}

bool FBoxBase::BuildData(const FBoxBuilderData& InOutBuilderData)
{
	// Build children recursively first.
	for(int32 i=0; i<Children.Num(); ++i)
	{
		Children[i]->BuildData(InOutBuilderData);
	}
	// If this is an artificial 'root' box, that is, the file level, then we are done.
	if (Type == 0)
	{
		return true;
	}

	FBoxDataWriter dw(CompiledBoxData);
	CompiledBoxData_Offset_Size = dw.PutAlignedValue(Size);
	dw.PutAlignedValue(Type);
	Size = dw.GetBytePosition();
	return true;
}

void FBoxBase::CalculateBoxSizes()
{
	for(int32 i=0; i<Children.Num(); ++i)
	{
		Children[i]->CalculateBoxSizes();
		Size += Children[i]->Size;
	}
	if (Type)
	{
		FBoxDataWriter dw(CompiledBoxData);
		dw.SetAlignedValueAt(Size, CompiledBoxData_Offset_Size);
	}
}



bool FBoxFull::BuildData(const FBoxBuilderData& InOutBuilderData)
{
	FBoxBase::BuildData(InOutBuilderData);
	FBoxDataWriter dw(CompiledBoxData);
	dw.PutAlignedValue(Version);
	dw.PutBits(Flags, 24);
	check(dw.IsByteAligned());
	Size = dw.GetBytePosition();
	return true;
}


bool FBoxFTYP::BuildData(const FBoxBuilderData& InOutBuilderData)
{
	FBoxBase::BuildData(InOutBuilderData);
	FBoxDataWriter dw(CompiledBoxData);
	MajorBrandPatchOffset = dw.PutAlignedValue(MajorBrand);
	dw.PutAlignedValue(MinorVersion);
	for(int32 i=0; i<CompatibleBrands.Num(); ++i)
	{
		dw.PutAlignedValue(CompatibleBrands[i]);
	}
	Size = dw.GetBytePosition();
	bWasBuilt = true;
	return true;
}

bool FBoxFTYP::UpdateMajorBrand(uint32 InMajorBrand)
{
	SetMajorBrand(InMajorBrand);
	if (!bWasBuilt)
	{
		return true;
	}
	FBoxDataWriter dw(CompiledBoxData);
	dw.SetAlignedValueAt(MajorBrand, MajorBrandPatchOffset);
	return true;
}
bool FBoxFTYP::RewriteChanges(FRewriteBoxChangeDelegate InRewriteBoxChangeDelegate)
{
	// If this box was not yet built then there's noting to rewrite.
	if (!bWasBuilt)
	{
		return true;
	}
	return InRewriteBoxChangeDelegate.Execute(AbsoluteFileOffset + MajorBrandPatchOffset, MakeConstArrayView(CompiledBoxData.GetData() + MajorBrandPatchOffset, 4));
}


bool FBoxFREE::BuildData(const FBoxBuilderData& InOutBuilderData)
{
	FBoxBase::BuildData(InOutBuilderData);
	FBoxDataWriter dw(CompiledBoxData);
	for(uint32 i=0; i<NumEmptyBytes; ++i)
	{
		dw.PutAlignedValue((uint8)0);
	}
	Size = dw.GetBytePosition();
	return true;
}

bool FBoxMVHD::BuildData(const FBoxBuilderData& InOutBuilderData)
{
	// Any of the values require a version 1 box?
	Version = (CreationTime > 0xffffffffU || ModificationTime > 0xffffffffU || (Duration != ~((uint64)0) && Duration > 0xffffffffU)) ? 1 : 0;

	FBoxFull::BuildData(InOutBuilderData);
	FBoxDataWriter dw(CompiledBoxData);
	if (Version == 1)
	{
		dw.PutAlignedValue(CreationTime);
		dw.PutAlignedValue(ModificationTime);
		TimescalePatchOffset = dw.PutAlignedValue(Timescale);
		DurationPatchOffset = dw.PutAlignedValue(Duration);
	}
	else
	{
		dw.PutAlignedValue((uint32)CreationTime);
		dw.PutAlignedValue((uint32)ModificationTime);
		TimescalePatchOffset = dw.PutAlignedValue(Timescale);
		DurationPatchOffset = dw.PutAlignedValue((uint32)Duration);
	}
	dw.PutAlignedValue((uint32)0x00010000);	// rate of 1.0
	dw.PutAlignedValue((uint16)0x0100);		// volume 1.0
	dw.PutAlignedValue((uint16)0);			// reserved
	dw.PutAlignedValue((uint32)0);			// reserved
	dw.PutAlignedValue((uint32)0);			// reserved
	// unity matrix
	dw.PutAlignedValue((uint32)0x10000);
	dw.PutAlignedValue((uint32)0);
	dw.PutAlignedValue((uint32)0);
	dw.PutAlignedValue((uint32)0);
	dw.PutAlignedValue((uint32)0x10000);
	dw.PutAlignedValue((uint32)0);
	dw.PutAlignedValue((uint32)0);
	dw.PutAlignedValue((uint32)0x10000);
	dw.PutAlignedValue((uint32)0x40000000);
	// predefined
	dw.PutAlignedValue((uint32)0);
	dw.PutAlignedValue((uint32)0);
	dw.PutAlignedValue((uint32)0);
	dw.PutAlignedValue((uint32)0);
	dw.PutAlignedValue((uint32)0);
	dw.PutAlignedValue((uint32)0);
	// next_track_ID
	dw.PutAlignedValue(NextTrackID);
	Size = dw.GetBytePosition();
	bWasBuilt = true;
	return true;
}
bool FBoxMVHD::UpdateTimescale(uint32 InTimescale)
{
	if (!bWasBuilt)
	{
		SetTimescale(InTimescale);
		return true;
	}
	SetTimescale(InTimescale);
	FBoxDataWriter dw(CompiledBoxData);
	dw.SetAlignedValueAt(Timescale, TimescalePatchOffset);
	return true;
}
bool FBoxMVHD::UpdateDuration(uint64 InDuration)
{
	if (!bWasBuilt)
	{
		SetDuration(InDuration);
		return true;
	}
	if (InDuration > 0xffffffffU && Version == 0)
	{
		return false;
	}
	SetDuration(InDuration);
	FBoxDataWriter dw(CompiledBoxData);
	if (Version == 1)
	{
		dw.SetAlignedValueAt(Duration, DurationPatchOffset);
	}
	else
	{
		dw.SetAlignedValueAt((uint32)Duration, DurationPatchOffset);
	}
	return true;
}
bool FBoxMVHD::RewriteChanges(FRewriteBoxChangeDelegate InRewriteBoxChangeDelegate)
{
	// If this box was not yet built then there's noting to rewrite.
	if (!bWasBuilt)
	{
		return true;
	}
	return InRewriteBoxChangeDelegate.Execute(AbsoluteFileOffset + TimescalePatchOffset, MakeConstArrayView(CompiledBoxData.GetData() + TimescalePatchOffset, 4)) &&
		   InRewriteBoxChangeDelegate.Execute(AbsoluteFileOffset + DurationPatchOffset, MakeConstArrayView(CompiledBoxData.GetData() + DurationPatchOffset, Version == 1 ? 8 : 4));
}


bool FBoxTKHD::BuildData(const FBoxBuilderData& InOutBuilderData)
{
	// Any of the values require a version 1 box?
	Version = (CreationTime > 0xffffffffU || ModificationTime > 0xffffffffU || (Duration != ~((uint64)0) && Duration > 0xffffffffU)) ? 1 : 0;

	Flags |= CustomFlags;
	FBoxFull::BuildData(InOutBuilderData);
	FBoxDataWriter dw(CompiledBoxData);
	if (Version == 1)
	{
		dw.PutAlignedValue(CreationTime);
		dw.PutAlignedValue(ModificationTime);
		dw.PutAlignedValue(TrackID);
		dw.PutAlignedValue((uint32)0);
		DurationPatchOffset = dw.PutAlignedValue(Duration);
	}
	else
	{
		dw.PutAlignedValue((uint32)CreationTime);
		dw.PutAlignedValue((uint32)ModificationTime);
		dw.PutAlignedValue(TrackID);
		dw.PutAlignedValue((uint32)0);
		DurationPatchOffset = dw.PutAlignedValue((uint32)Duration);
	}
	// reserved
	dw.PutAlignedValue((uint32)0);
	dw.PutAlignedValue((uint32)0);
	dw.PutAlignedValue(Layer);
	dw.PutAlignedValue(AlternateGroup);
	dw.PutAlignedValue(Volume);
	dw.PutAlignedValue((uint16)0);
	// matrix
	dw.PutAlignedValue(Matrix[0]);
	dw.PutAlignedValue(Matrix[1]);
	dw.PutAlignedValue(Matrix[2]);
	dw.PutAlignedValue(Matrix[3]);
	dw.PutAlignedValue(Matrix[4]);
	dw.PutAlignedValue(Matrix[5]);
	dw.PutAlignedValue(Matrix[6]);
	dw.PutAlignedValue(Matrix[7]);
	dw.PutAlignedValue(Matrix[8]);
	dw.PutAlignedValue(Width);
	dw.PutAlignedValue(Height);
	Size = dw.GetBytePosition();
	bWasBuilt = true;
	return true;
}
bool FBoxTKHD::UpdateDuration(uint64 InDuration)
{
	if (!bWasBuilt)
	{
		SetDuration(InDuration);
		return true;
	}
	// If the duration now requires 64 bit wide fields when originally we had only 32 then this cannot be updated.
	if (InDuration > 0xffffffffU && Version == 0)
	{
		return false;
	}
	SetDuration(InDuration);
	FBoxDataWriter dw(CompiledBoxData);
	if (Version == 1)
	{
		dw.SetAlignedValueAt(Duration, DurationPatchOffset);
	}
	else
	{
		dw.SetAlignedValueAt((uint32)Duration, DurationPatchOffset);
	}
	return true;
}
bool FBoxTKHD::RewriteChanges(FRewriteBoxChangeDelegate InRewriteBoxChangeDelegate)
{
	// If this box was not yet built then there's noting to rewrite.
	if (!bWasBuilt)
	{
		return true;
	}
	return InRewriteBoxChangeDelegate.Execute(AbsoluteFileOffset + DurationPatchOffset, MakeConstArrayView(CompiledBoxData.GetData() + DurationPatchOffset, Version == 1 ? 8 : 4));
}




bool FBoxMDHD::BuildData(const FBoxBuilderData& InOutBuilderData)
{
	// Any of the values require a version 1 box?
	Version = (CreationTime > 0xffffffffU || ModificationTime > 0xffffffffU || (Duration != ~((uint64)0) && Duration > 0xffffffffU)) ? 1 : 0;

	FBoxFull::BuildData(InOutBuilderData);
	FBoxDataWriter dw(CompiledBoxData);
	if (Version == 1)
	{
		dw.PutAlignedValue(CreationTime);
		dw.PutAlignedValue(ModificationTime);
		dw.PutAlignedValue(Timescale);
		DurationPatchOffset = dw.PutAlignedValue(Duration);
	}
	else
	{
		dw.PutAlignedValue((uint32)CreationTime);
		dw.PutAlignedValue((uint32)ModificationTime);
		dw.PutAlignedValue(Timescale);
		DurationPatchOffset = dw.PutAlignedValue((uint32)Duration);
	}
	dw.PutBits(0, 1);
	dw.PutBits(LanguageCode[0], 5);
	dw.PutBits(LanguageCode[1], 5);
	dw.PutBits(LanguageCode[2], 5);
	check(dw.IsByteAligned());
	dw.PutAlignedValue((uint16)0);
	Size = dw.GetBytePosition();
	bWasBuilt = true;
	return true;
}
bool FBoxMDHD::UpdateDuration(uint64 InDuration)
{
	if (!bWasBuilt)
	{
		SetDuration(InDuration);
		return true;
	}
	// If the duration now requires 64 bit wide fields when originally we had only 32 then this cannot be updated.
	if (InDuration > 0xffffffffU && Version == 0)
	{
		return false;
	}
	SetDuration(InDuration);
	FBoxDataWriter dw(CompiledBoxData);
	if (Version == 1)
	{
		dw.SetAlignedValueAt(Duration, DurationPatchOffset);
	}
	else
	{
		dw.SetAlignedValueAt((uint32)Duration, DurationPatchOffset);
	}
	return true;
}
bool FBoxMDHD::RewriteChanges(FRewriteBoxChangeDelegate InRewriteBoxChangeDelegate)
{
	// If this box was not yet built then there's noting to rewrite.
	if (!bWasBuilt)
	{
		return true;
	}
	return InRewriteBoxChangeDelegate.Execute(AbsoluteFileOffset + DurationPatchOffset, MakeConstArrayView(CompiledBoxData.GetData() + DurationPatchOffset, Version == 1 ? 8 : 4));
}


bool FBoxHDLR::BuildData(const FBoxBuilderData& InOutBuilderData)
{
	FBoxFull::BuildData(InOutBuilderData);
	FBoxDataWriter dw(CompiledBoxData);
	dw.PutAlignedValue((uint32)0);
	dw.PutAlignedValue(HandlerType);
	dw.PutAlignedValue((uint32)0);
	dw.PutAlignedValue((uint32)0);
	dw.PutAlignedValue((uint32)0);
	FTCHARToUTF8 cnv(*Name);
	int32 Len = cnv.Length();
	auto Ptr = cnv.Get();
	for(int32 i=0; i<Len; ++i)
	{
		dw.PutAlignedValue((uint8)*Ptr++);
	}
	dw.PutAlignedValue((uint8)0);
	Size = dw.GetBytePosition();
	return true;
}


bool FBoxVMHD::BuildData(const FBoxBuilderData& InOutBuilderData)
{
	Flags = 1;
	FBoxFull::BuildData(InOutBuilderData);
	FBoxDataWriter dw(CompiledBoxData);
	dw.PutAlignedValue(GraphicsMode);
	dw.PutAlignedValue(OpColor[0]);
	dw.PutAlignedValue(OpColor[1]);
	dw.PutAlignedValue(OpColor[2]);
	Size = dw.GetBytePosition();
	return true;
}


bool FBoxSMHD::BuildData(const FBoxBuilderData& InOutBuilderData)
{
	FBoxFull::BuildData(InOutBuilderData);
	FBoxDataWriter dw(CompiledBoxData);
	dw.PutAlignedValue(Balance);
	dw.PutAlignedValue((uint16)0);
	Size = dw.GetBytePosition();
	return true;
}


bool FBoxDREF::BuildData(const FBoxBuilderData& InOutBuilderData)
{
	FBoxFull::BuildData(InOutBuilderData);
	FBoxDataWriter dw(CompiledBoxData);
	dw.PutAlignedValue((uint32)Children.Num());
	Size = dw.GetBytePosition();
	return true;
}


bool FBoxURL::BuildData(const FBoxBuilderData& InOutBuilderData)
{
	Flags = 1;
	FBoxFull::BuildData(InOutBuilderData);
	FBoxDataWriter dw(CompiledBoxData);
	Size = dw.GetBytePosition();
	return true;
}


bool FBoxSampleEntry::BuildData(const FBoxBuilderData& InOutBuilderData)
{
	FBoxBase::BuildData(InOutBuilderData);
	FBoxDataWriter dw(CompiledBoxData);
	dw.PutAlignedValue((uint8)0);				// reserved
	dw.PutAlignedValue((uint8)0);				// reserved
	dw.PutAlignedValue((uint8)0);				// reserved
	dw.PutAlignedValue((uint8)0);				// reserved
	dw.PutAlignedValue((uint8)0);				// reserved
	dw.PutAlignedValue((uint8)0);				// reserved
	dw.PutAlignedValue(DataReferenceIndex);		// data_reference_index
	Size = dw.GetBytePosition();
	return true;
}

bool FBoxVisualSampleEntry::BuildData(const FBoxBuilderData& InOutBuilderData)
{
	FBoxSampleEntry::BuildData(InOutBuilderData);
	FBoxDataWriter dw(CompiledBoxData);
	dw.PutAlignedValue((uint16)0);				// pre_defined
	dw.PutAlignedValue((uint16)0);				// reserved
	dw.PutAlignedValue((uint32)0);				// pre_defined
	dw.PutAlignedValue((uint32)0);				// pre_defined
	dw.PutAlignedValue((uint32)0);				// pre_defined
	dw.PutAlignedValue(Width);
	dw.PutAlignedValue(Height);
	dw.PutAlignedValue((uint32)0x00480000U);	// horzresolution 72dpi
	dw.PutAlignedValue((uint32)0x00480000U);	// vertresolution 72dpi
	dw.PutAlignedValue((uint32)0);				// reserved
	dw.PutAlignedValue((uint16)1);				// frame_count

	FTCHARToUTF8 cnv(*CompressorName);
	int32 Len = cnv.Length();
	Len = Len > 31 ? 31 : Len;
	dw.PutAlignedValue((uint8)Len);
	auto Ptr = cnv.Get();
	int32 i;
	for(i=0; i<Len; ++i)
	{
		dw.PutAlignedValue((uint8)*Ptr++);
	}
	for(; i<31; ++i)
	{
		dw.PutAlignedValue((uint8)0);
	}
	dw.PutAlignedValue((uint16)0x18);			// depth
	dw.PutAlignedValue((uint16)0xffff);			// pre_defined
	for(i=0; i<AdditionalBoxes.Num(); ++i)
	{
		dw.PutAlignedArray(AdditionalBoxes[i]);
	}
	Size = dw.GetBytePosition();
	return true;
}


bool FBoxAudioSampleEntry::BuildData(const FBoxBuilderData& InOutBuilderData)
{
	FBoxSampleEntry::BuildData(InOutBuilderData);
	FBoxDataWriter dw(CompiledBoxData);
	// Need an `AudioSampleEntryV1` instead?
	const bool bV1 = SampleRate >= 0x10000;
	dw.PutAlignedValue((uint16)(bV1 ? 1U : 0U));// version
	dw.PutAlignedValue((uint16)0);				// reserved
	dw.PutAlignedValue((uint32)0);				// reserved
	dw.PutAlignedValue(ChannelCount);
	dw.PutAlignedValue((uint16)16);				// samplesize == 16
	dw.PutAlignedValue((uint16)0);				// reserved
	dw.PutAlignedValue((uint16)0);				// reserved
	dw.PutAlignedValue((uint32)(bV1 ? 0x10000U : (SampleRate << 16)));
	// Add an `srat` box, which is full box version=0,flags=0
	if (bV1)
	{
		dw.PutAlignedValue((uint32)16U);
		dw.PutAlignedValue(BoxAtom('s','r','a','t'));
		dw.PutAlignedValue((uint32)0U);
		dw.PutAlignedValue(SampleRate);
	}
	for(int32 i=0; i<AdditionalBoxes.Num(); ++i)
	{
		dw.PutAlignedArray(AdditionalBoxes[i]);
	}
	Size = dw.GetBytePosition();
	return true;
}


bool FBoxTimecodeSampleEntry::BuildData(const FBoxBuilderData& InOutBuilderData)
{
	FBoxSampleEntry::BuildData(InOutBuilderData);
	FBoxDataWriter dw(CompiledBoxData);
	uint32 tcFlags = 0;
	tcFlags |= bDropFrame ? 0x1U : 0x0U;
	tcFlags |= bMax24Hours ? 0x2U : 0x0U;
	tcFlags |= bAllowNegativeTimes ? 0x4U : 0x0U;
	// Note: Do not set flag 0x8 to indicate counter values.
	//  It is always counter values nowadays.
	dw.PutAlignedValue((uint32)0);				// reserved
	dw.PutAlignedValue(tcFlags);
	dw.PutAlignedValue((uint32)Timescale);
	dw.PutAlignedValue((uint32)FrameDuration);
	dw.PutAlignedValue((uint8)FramesPerSecond);
	dw.PutAlignedValue((uint8)0U);				// reserved
	for(int32 i=0; i<AdditionalBoxes.Num(); ++i)
	{
		dw.PutAlignedArray(AdditionalBoxes[i]);
	}
	Size = dw.GetBytePosition();
	return true;
}


bool FBoxSTSD::BuildData(const FBoxBuilderData& InOutBuilderData)
{
	FBoxFull::BuildData(InOutBuilderData);
	FBoxDataWriter dw(CompiledBoxData);
	dw.PutAlignedValue((uint32)Children.Num());
	Size = dw.GetBytePosition();
	return true;
}


bool FBoxSTTS::BuildData(const FBoxBuilderData& InOutBuilderData)
{
	FBoxFull::BuildData(InOutBuilderData);
	FBoxDataWriter dw(CompiledBoxData);
	dw.PutAlignedValue((uint32)SampleDurations.Num());
	for(int32 i=0, iMax=SampleDurations.Num(); i<iMax; ++i)
	{
		const FEntry& e(SampleDurations[i]);
		dw.PutAlignedValue(e.Count);
		dw.PutAlignedValue(e.Delta);
	}
	Size = dw.GetBytePosition();
	return true;
}


bool FBoxSTSS::BuildData(const FBoxBuilderData& InOutBuilderData)
{
	FBoxFull::BuildData(InOutBuilderData);
	FBoxDataWriter dw(CompiledBoxData);
	dw.PutAlignedValue((uint32)SyncSampleIndices.Num());
	for(int32 i=0, iMax=SyncSampleIndices.Num(); i<iMax; ++i)
	{
		dw.PutAlignedValue(SyncSampleIndices[i]);
	}
	Size = dw.GetBytePosition();
	return true;
}


bool FBoxCTTS::BuildData(const FBoxBuilderData& InOutBuilderData)
{
	Version = bHasNegativeEntries ? 1 : 0;
	FBoxFull::BuildData(InOutBuilderData);
	FBoxDataWriter dw(CompiledBoxData);
	dw.PutAlignedValue((uint32)CompositionTimeOffsets.Num());
	if (Version == 0)
	{
		for(int32 i=0, iMax=CompositionTimeOffsets.Num(); i<iMax; ++i)
		{
			const FEntry& e(CompositionTimeOffsets[i]);
			dw.PutAlignedValue(e.Count);
			dw.PutAlignedValue((uint32)e.Delta);
		}
	}
	else
	{
		for(int32 i=0, iMax=CompositionTimeOffsets.Num(); i<iMax; ++i)
		{
			const FEntry& e(CompositionTimeOffsets[i]);
			dw.PutAlignedValue(e.Count);
			dw.PutAlignedValue(e.Delta);
		}
	}
	Size = dw.GetBytePosition();
	return true;
}


bool FBoxSTSC::BuildData(const FBoxBuilderData& InOutBuilderData)
{
	FBoxFull::BuildData(InOutBuilderData);
	FBoxDataWriter dw(CompiledBoxData);
	dw.PutAlignedValue((uint32)SampleToChunkEntries.Num());
	for(int32 i=0, iMax=SampleToChunkEntries.Num(); i<iMax; ++i)
	{
		const FEntry& e(SampleToChunkEntries[i]);
		dw.PutAlignedValue(e.first_chunk);
		dw.PutAlignedValue(e.samples_per_chunk);
		dw.PutAlignedValue(e.sample_description_index);
	}
	Size = dw.GetBytePosition();
	return true;
}


bool FBoxSTSZ::BuildData(const FBoxBuilderData& InOutBuilderData)
{
	if (bUseSmallSizeBox && ConstantSampleSize == 0 && LargestSampleSize < 0x10000)
	{
		Type = BoxAtom('s','t','z','2');
		uint8 NumBits = LargestSampleSize >= 256 ? 16 : LargestSampleSize >= 16 ? 8 : 4;
		FBoxFull::BuildData(InOutBuilderData);
		FBoxDataWriter dw(CompiledBoxData);
		dw.PutAlignedValue((uint32)NumBits);
		dw.PutAlignedValue((uint32)SampleSizes.Num());
		if (NumBits == 16)
		{
			for(int32 i=0,iMax=SampleSizes.Num(); i<iMax; ++i)
			{
				dw.PutAlignedValue((uint16) SampleSizes[i]);
			}
		}
		else if (NumBits == 8)
		{
			for(int32 i=0,iMax=SampleSizes.Num(); i<iMax; ++i)
			{
				dw.PutAlignedValue((uint8) SampleSizes[i]);
			}
		}
		else// if (NumBits == 4)
		{
			for(int32 i=0,iMax=SampleSizes.Num() & ~1; i<iMax; i+=2)
			{
				uint8 v = ((SampleSizes[i+1] & 15) << 4) | (SampleSizes[i] & 15);
				dw.PutAlignedValue(v);
			}
			if (SampleSizes.Num() & 1)
			{
				uint8 v = (SampleSizes[SampleSizes.Num()-1] & 15) << 4;
				dw.PutAlignedValue(v);
			}
		}
		Size = dw.GetBytePosition();
	}
	else
	{
		FBoxFull::BuildData(InOutBuilderData);
		FBoxDataWriter dw(CompiledBoxData);
		if (ConstantSampleSize == 0 && !bHasVaryingSizes && SampleSizes.Num())
		{
			ConstantSampleSize = SampleSizes[0];
		}
		dw.PutAlignedValue(ConstantSampleSize);
		dw.PutAlignedValue((uint32)SampleSizes.Num());
		if (ConstantSampleSize == 0)
		{
			for(int32 i=0,iMax=SampleSizes.Num(); i<iMax; ++i)
			{
				dw.PutAlignedValue(SampleSizes[i]);
			}
		}
		Size = dw.GetBytePosition();
	}
	return true;
}


bool FBoxSTCO::AddFileOffset(int64 InOffsetToAdd)
{
	for(int32 i=0, iMax=ChunkOffsets.Num(); i<iMax; ++i)
	{
		int64 Offset = ChunkOffsets[i] + InOffsetToAdd;
		if (Offset < 0 || (!bNeeds64Bits && Offset > 0xffffffff))
		{
			return false;
		}
		ChunkOffsets[i] = Offset;
	}
	return true;
}

bool FBoxSTCO::BuildData(const FBoxBuilderData& InOutBuilderData)
{
	if (bNeeds64Bits)
	{
		Type = BoxAtom('c','o','6','4');
	}
	FBoxFull::BuildData(InOutBuilderData);
	FBoxDataWriter dw(CompiledBoxData);
	dw.PutAlignedValue((uint32)ChunkOffsets.Num());
	if (bNeeds64Bits)
	{
		for(int32 i=0,iMax=ChunkOffsets.Num(); i<iMax; ++i)
		{
			dw.PutAlignedValue((uint64)ChunkOffsets[i]);
		}
	}
	else
	{
		for(int32 i=0,iMax=ChunkOffsets.Num(); i<iMax; ++i)
		{
			dw.PutAlignedValue((uint32)ChunkOffsets[i]);
		}
	}
	Size = dw.GetBytePosition();
	return true;
}


bool FBoxMEHD::BuildData(const FBoxBuilderData& InOutBuilderData)
{
	Version = FragmentDuration > 0xffffffffU ? 1 : 0;
	FBoxFull::BuildData(InOutBuilderData);
	FBoxDataWriter dw(CompiledBoxData);
	if (Version == 1)
	{
		FragmentDurationPatchOffset = dw.PutAlignedValue((uint64) FragmentDuration);
	}
	else
	{
		FragmentDurationPatchOffset = dw.PutAlignedValue((uint32) FragmentDuration);
	}
	Size = dw.GetBytePosition();
	bWasBuilt = true;
	return true;
}
bool FBoxMEHD::UpdateFragmentDuration(uint64 InFragmentDuration)
{
	if (!bWasBuilt)
	{
		SetFragmentDuration(InFragmentDuration);
		return true;
	}
	// If the duration now requires 64 bit wide fields when originally we had only 32 then this cannot be updated.
	if (InFragmentDuration > 0xffffffffU && Version == 0)
	{
		return false;
	}
	SetFragmentDuration(InFragmentDuration);
	FBoxDataWriter dw(CompiledBoxData);
	if (Version == 1)
	{
		dw.SetAlignedValueAt(FragmentDuration, FragmentDurationPatchOffset);
	}
	else
	{
		dw.SetAlignedValueAt((uint32)FragmentDuration, FragmentDurationPatchOffset);
	}
	return true;
}
bool FBoxMEHD::RewriteChanges(FRewriteBoxChangeDelegate InRewriteBoxChangeDelegate)
{
	// If this box was not yet built then there's noting to rewrite.
	if (!bWasBuilt)
	{
		return true;
	}
	return InRewriteBoxChangeDelegate.Execute(AbsoluteFileOffset + FragmentDurationPatchOffset, MakeConstArrayView(CompiledBoxData.GetData() + FragmentDurationPatchOffset, Version == 1 ? 8 : 4));
}


bool FBoxTREX::BuildData(const FBoxBuilderData& InOutBuilderData)
{
	FBoxFull::BuildData(InOutBuilderData);
	FBoxDataWriter dw(CompiledBoxData);
	dw.PutAlignedValue(TrackID);
	dw.PutAlignedValue(default_sample_description_index);
	dw.PutAlignedValue(default_sample_duration);
	dw.PutAlignedValue(default_sample_size);
	dw.PutAlignedValue(default_sample_flags);
	Size = dw.GetBytePosition();
	return true;
}


bool FBoxMFHD::BuildData(const FBoxBuilderData& InOutBuilderData)
{
	FBoxFull::BuildData(InOutBuilderData);
	FBoxDataWriter dw(CompiledBoxData);
	dw.PutAlignedValue(SequenceNumber);
	Size = dw.GetBytePosition();
	return true;
}


bool FBoxTFHD::BuildData(const FBoxBuilderData& InOutBuilderData)
{
	if (base_data_offset.IsSet())
	{
		Flags |= 0x1;
	}
	else if (bDefaultBaseIsMoof)
	{
		Flags |= 0x20000;
	}
	if (sample_description_index.IsSet())
	{
		Flags |= 0x2;
	}
	if (default_sample_duration.IsSet())
	{
		Flags |= 0x8;
	}
	if (default_sample_size.IsSet())
	{
		Flags |= 0x10;
	}
	if (default_sample_flags.IsSet())
	{
		Flags |= 0x20;
	}

	FBoxFull::BuildData(InOutBuilderData);
	FBoxDataWriter dw(CompiledBoxData);
	dw.PutAlignedValue(TrackID);
	if (base_data_offset.IsSet())
	{
		BaseDataPatchOffset = dw.PutAlignedValue(base_data_offset.GetValue());
	}
	if (sample_description_index.IsSet())
	{
		dw.PutAlignedValue(sample_description_index.GetValue());
	}
	if (default_sample_duration.IsSet())
	{
		dw.PutAlignedValue(default_sample_duration.GetValue());
	}
	if (default_sample_size.IsSet())
	{
		dw.PutAlignedValue(default_sample_size.GetValue());
	}
	if (default_sample_flags.IsSet())
	{
		dw.PutAlignedValue(default_sample_flags.GetValue());
	}
	Size = dw.GetBytePosition();
	bWasBuilt = true;
	return true;
}
bool FBoxTFHD::UpdateBaseDataOffset(uint64 InBaseDataOffset)
{
	if (!bWasBuilt)
	{
		SetBaseDataOffset(InBaseDataOffset);
		return true;
	}
	// If the base data offset was not set before it cannot be set now.
	if (!base_data_offset.IsSet())
	{
		return false;
	}
	SetBaseDataOffset(InBaseDataOffset);
	FBoxDataWriter dw(CompiledBoxData);
	dw.SetAlignedValueAt(base_data_offset.GetValue(), BaseDataPatchOffset);
	return true;
}
bool FBoxTFHD::RewriteChanges(FRewriteBoxChangeDelegate InRewriteBoxChangeDelegate)
{
	// If this box was not yet built then there's noting to rewrite.
	if (!bWasBuilt)
	{
		return true;
	}
	return InRewriteBoxChangeDelegate.Execute(AbsoluteFileOffset + BaseDataPatchOffset, MakeConstArrayView(CompiledBoxData.GetData() + BaseDataPatchOffset, 8));
}


bool FBoxTFDT::BuildData(const FBoxBuilderData& InOutBuilderData)
{
	Version = BaseMediaDecodeTime > 0xffffffffU ? 1 : 0;
	FBoxFull::BuildData(InOutBuilderData);
	FBoxDataWriter dw(CompiledBoxData);
	if (Version == 1)
	{
		dw.PutAlignedValue(BaseMediaDecodeTime);
	}
	else
	{
		dw.PutAlignedValue((uint32)BaseMediaDecodeTime);
	}
	Size = dw.GetBytePosition();
	return true;
}


bool FBoxTRUN::BuildData(const FBoxBuilderData& InOutBuilderData)
{
	Version = bRequiresCompositionTimeOffsets ? 1 : 0;
	if (DataOffset.IsSet())
	{
		Flags |= 0x1;
	}
	// If the first sample has dedicated flags then no flags for individual samples must be used!
	// See ISO/IEC 14496-12:2022 - 8.8.8.1 Definition
	if (first_sample_flags.IsSet())
	{
		Flags |= 0x4;
	}
	else if (!bDoNotWriteFlags)
	{
		Flags |= 0x400;
	}

	if (bDoNotWriteDurations == false && (bHasDifferentDurations || Samples.Num() == 1))
	{
		Flags |= 0x100;
	}
	if (bDoNotWriteSizes == false && (bHasDifferentSizes || Samples.Num() == 1))
	{
		Flags |= 0x200;
	}
	if (bRequiresCompositionTimeOffsets)
	{
		Flags |= 0x800;
	}

	FBoxFull::BuildData(InOutBuilderData);
	FBoxDataWriter dw(CompiledBoxData);
	dw.PutAlignedValue((uint32)Samples.Num());
	if ((Flags & 0x1) != 0)
	{
		DataOffsetPatchOffset = dw.PutAlignedValue(DataOffset.GetValue());
	}
	if ((Flags & 0x4) != 0)
	{
		dw.PutAlignedValue(first_sample_flags.GetValue());
	}
	// Only write sample information if any of them are to be.
	// If everything uses default values then there is no need for that.
	if ((Flags & 0xf00) != 0)
	{
		for(int32 i=0, iMax=Samples.Num(); i<iMax; ++i)
		{
			const FEntry& e(Samples[i]);
			if (Flags & 0x100)
			{
				dw.PutAlignedValue(e.sample_duration);
			}
			if (Flags & 0x200)
			{
				dw.PutAlignedValue(e.sample_size);
			}
			if (Flags & 0x400)
			{
				dw.PutAlignedValue(e.sample_flags);
			}
			if (Flags & 0x800)
			{
				dw.PutAlignedValue(e.sample_composition_time_offset);
			}
		}
	}
	Size = dw.GetBytePosition();
	return true;
}


bool FBoxSUBS::BuildData(const FBoxBuilderData& InOutBuilderData)
{
	Version = 1;	// always write 32 bit subsample sizes
	FBoxFull::BuildData(InOutBuilderData);
	FBoxDataWriter dw(CompiledBoxData);
	dw.PutAlignedValue((uint32)SubSamples.Num());
	uint32 PreviousSampleNum = 0;
	for(int32 i=0, iMax=SubSamples.Num(); i<iMax; ++i)
	{
		uint32 SampleDelta = SubSamples[i].SampleNumber - PreviousSampleNum;
		PreviousSampleNum = SubSamples[i].SampleNumber;
		dw.PutAlignedValue(SampleDelta);
		dw.PutAlignedValue((uint16) SubSamples[i].Subs.Num());
		for(int32 j=0, jMax=SubSamples[i].Subs.Num(); j<jMax; ++j)
		{
			if (Version == 1)
			{
				dw.PutAlignedValue((uint32) SubSamples[i].Subs[j].subsample_size);
			}
			else
			{
				dw.PutAlignedValue((uint16) SubSamples[i].Subs[j].subsample_size);
			}
			dw.PutAlignedValue((uint8) SubSamples[i].Subs[j].subsample_priority);
			dw.PutAlignedValue((uint8) SubSamples[i].Subs[j].discardable);
			dw.PutAlignedValue((uint32) SubSamples[i].Subs[j].codec_specific_parameters);
		}
	}
	Size = dw.GetBytePosition();
	return true;
}


bool FBoxELST::BuildData(const FBoxBuilderData& InOutBuilderData)
{
	Version = EditDuration > TNumericLimits<uint32>::Max() || MediaTime > TNumericLimits<int32>::Max() || MediaTime < TNumericLimits<int32>::Min() ? 1 : 0;
	FBoxFull::BuildData(InOutBuilderData);
	FBoxDataWriter dw(CompiledBoxData);
	dw.PutAlignedValue((uint32)1);	// just one entry.
	if (Version == 1)
	{
		EditDurationPatchOffset = dw.PutAlignedValue(EditDuration);
		dw.PutAlignedValue(MediaTime);
	}
	else
	{
		EditDurationPatchOffset = dw.PutAlignedValue((uint32)EditDuration);
		dw.PutAlignedValue((int32)MediaTime);
	}
	dw.PutAlignedValue((int16)1);
	dw.PutAlignedValue((int16)0);
	Size = dw.GetBytePosition();
	bWasBuilt = true;
	return true;
}

bool FBoxELST::UpdateEditDuration(uint64 InEditDuration)
{
	if (!bWasBuilt)
	{
		SetEditDuration(InEditDuration);
		return true;
	}
	if (InEditDuration > TNumericLimits<uint32>::Max() && Version == 0)
	{
		return false;
	}
	SetEditDuration(InEditDuration);
	FBoxDataWriter dw(CompiledBoxData);
	if (Version == 1)
	{
		dw.SetAlignedValueAt(EditDuration, EditDurationPatchOffset);
	}
	else
	{
		dw.SetAlignedValueAt((uint32)EditDuration, EditDurationPatchOffset);
	}
	return true;
}
bool FBoxELST::RewriteChanges(FRewriteBoxChangeDelegate InRewriteBoxChangeDelegate)
{
	// If this box was not yet built then there's noting to rewrite.
	if (!bWasBuilt)
	{
		return true;
	}
	return InRewriteBoxChangeDelegate.Execute(AbsoluteFileOffset + EditDurationPatchOffset, MakeConstArrayView(CompiledBoxData.GetData() + EditDurationPatchOffset, Version == 1 ? 8 : 4));
}


bool FBoxNAMEbase::BuildData(const FBoxBuilderData& InOutBuilderData)
{
	// Convert the name into an UTF8 array
	TArray<uint8> UTF8Data;
	FTCHARToUTF8 cnv(*Name);
	int32 Len = cnv.Length();
	UTF8Data.AddUninitialized(Len);
	FMemory::Memcpy(UTF8Data.GetData(), cnv.Get(), Len);

	FBoxBase::BuildData(InOutBuilderData);
	FBoxDataWriter dw(CompiledBoxData);
	dw.PutAlignedArray(UTF8Data);
	Size = dw.GetBytePosition();
	return true;
}

bool FBoxNAME::BuildData(const FBoxBuilderData& InOutBuilderData)
{
	// Convert the name into an UTF8 array
	TArray<uint8> UTF8Data;
	FTCHARToUTF8 cnv(*Name);
	int32 Len = cnv.Length();
	UTF8Data.AddUninitialized(Len);
	FMemory::Memcpy(UTF8Data.GetData(), cnv.Get(), Len);

	FBoxFull::BuildData(InOutBuilderData);
	FBoxDataWriter dw(CompiledBoxData);
	dw.PutAlignedArray(UTF8Data);
	Size = dw.GetBytePosition();
	return true;
}


bool FBoxTREFType::BuildData(const FBoxBuilderData& InOutBuilderData)
{
	FBoxBase::SetType(ReferenceType);
	FBoxBase::BuildData(InOutBuilderData);
	FBoxDataWriter dw(CompiledBoxData);
	for(int32 i=0; i<ReferencedTrackNumbers.Num(); ++i)
	{
		dw.PutAlignedValue(ReferencedTrackNumbers[i]);
	}
	Size = dw.GetBytePosition();
	return true;
}


bool FBoxOpaqueData::BuildData(const FBoxBuilderData& InOutBuilderData)
{
	//FBoxBase::SetType(ReferenceType);
	FBoxBase::BuildData(InOutBuilderData);
	FBoxDataWriter dw(CompiledBoxData);
	dw.PutAlignedArray(RawBoxData);
	Size = dw.GetBytePosition();
	return true;
}


}
