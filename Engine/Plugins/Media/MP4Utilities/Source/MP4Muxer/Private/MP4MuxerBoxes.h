// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/ChunkedArray.h"
#include "MP4Muxer.h"

namespace MP4MuxerBoxes
{
static constexpr uint16 EndianSwap(uint16 value)			{ return (value >> 8) | (value << 8); }
static constexpr int16 EndianSwap(int16 value)				{ return int16(EndianSwap(uint16(value))); }
static constexpr uint32 EndianSwap(uint32 value)			{ return (value << 24) | ((value & 0xff00) << 8) | ((value >> 8) & 0xff00) | (value >> 24); }
static constexpr int32 EndianSwap(int32 value)				{ return int32(EndianSwap(uint32(value))); }
static constexpr uint64 EndianSwap(uint64 value)			{ return (uint64(EndianSwap(uint32(value & 0xffffffffU))) << 32) | uint64(EndianSwap(uint32(value >> 32))); }
static constexpr int64 EndianSwap(int64 value)				{ return int64(EndianSwap(uint64(value)));}

static constexpr uint32 BoxAtom(const uint8 A, const uint8 B, const uint8 C, const uint8 D)
{ return (static_cast<uint32>(A) << 24) | (static_cast<uint32>(B) << 16) | (static_cast<uint32>(C) << 8) | static_cast<uint32>(D); }


class FBoxBuilderData
{
public:
	FBoxBuilderData() = default;
	~FBoxBuilderData() = default;
};

class FBoxBase
{
public:
	FBoxBase(uint32 InType) : Type(InType)
	{ }
	virtual ~FBoxBase();
	virtual void ClearBuiltBoxData();
	virtual bool BuildData(const FBoxBuilderData& InOutBuilderData);
	virtual void CalculateBoxSizes();
	void AddChild(FBoxBase* InChild)
	{
		InChild->Parent = this;
		Children.Add(InChild);
	}
	void AddChildAfter(FBoxBase* InChild, FBoxBase* InAfter)
	{
		InChild->Parent = this;
		int32 Pos = Children.IndexOfByKey(InAfter);
		if (Pos != INDEX_NONE && Pos+1<Children.Num())
		{
			Children.Insert(InChild, Pos+1);
		}
		else
		{
			Children.Add(InChild);
		}
	}
	FBoxBase* GetParent() const
	{ return Parent; }
	void RemoveChild(FBoxBase* InChild)
	{ Children.Remove(InChild); }
	template <typename Predicate>
	bool VisitForWriting(Predicate Pred)
	{
		if (!::Invoke(Pred, this))
		{
			return false;
		}
		for(int32 i=0; i<Children.Num(); ++i)
		{
			if (!Children[i]->VisitForWriting(Pred))
			{
				return false;
			}
		}
		return true;
	}
	int64 GetSize() const
	{ return Size; }
	const TArray<uint8>& GetCompiledBoxData() const
	{ return CompiledBoxData; }
	// Gets set while writing
	void SetAbsoluteFileOffset(int64 InAbsoluteFileOffset)
	{ AbsoluteFileOffset = InAbsoluteFileOffset; }
	int64 GetAbsoluteFileOffset() const
	{ return AbsoluteFileOffset; }

	DECLARE_DELEGATE_RetVal_TwoParams(bool, FRewriteBoxChangeDelegate, int64 /*FilePosition*/, TConstArrayView<uint8> /*NewData*/);
	virtual bool RewriteChanges(FRewriteBoxChangeDelegate InRewriteBoxChangeDelegate)
	{ return true; }

protected:
	void SetType(uint32 InType)
	{ Type = InType; }
	void SetSize(uint32 InSize)
	{ Size = InSize; }
	void AddSizeOfChildren();
	uint32 Type = 0;
	uint32 Size = 0;
	FBoxBase* Parent = nullptr;
	TArray<FBoxBase*> Children;
	TArray<uint8> CompiledBoxData;
	uint32 CompiledBoxData_Offset_Size = 0;
	int64 AbsoluteFileOffset = -1;
	bool bWasBuilt = false;
};

class FBoxFull : public FBoxBase
{
public:
	FBoxFull(uint32 InType) : FBoxBase(InType)
	{ }
	virtual ~FBoxFull() = default;
	bool BuildData(const FBoxBuilderData& InOutBuilderData) override;
	void SetFlags(uint32 InFlags)
	{ Flags = InFlags & 0x00ffffff; }
protected:
	uint8 Version = 0;
	uint32 Flags = 0;
};


class FBoxFTYP : public FBoxBase
{
public:
	FBoxFTYP() : FBoxBase(BoxAtom('f','t','y','p'))
	{ }
	virtual ~FBoxFTYP() = default;

	void SetMajorBrand(uint32 InMajorBrand)
	{ MajorBrand = InMajorBrand; }
	void SetMinorVersion(uint32 InMinorVersion)
	{ MinorVersion = InMinorVersion; }
	void AddCompatibleBrand(uint32 InCompatibleBrand)
	{ CompatibleBrands.Emplace(InCompatibleBrand); }

	bool BuildData(const FBoxBuilderData& InOutBuilderData) override;
	bool UpdateMajorBrand(uint32 InMajorBrand);
	bool RewriteChanges(FRewriteBoxChangeDelegate InRewriteBoxChangeDelegate) override;
protected:
	uint32 MajorBrand = 0;
	uint32 MinorVersion = 0;
	TArray<uint32> CompatibleBrands;
	int64 MajorBrandPatchOffset = -1;
};


class FBoxFREE : public FBoxBase
{
public:
	FBoxFREE() : FBoxBase(BoxAtom('f','r','e','e'))
	{ }
	virtual ~FBoxFREE() = default;
	void SetNumberOfEmptyBytes(uint32 InNumEmptyBytes)
	{ NumEmptyBytes = InNumEmptyBytes; }
	bool BuildData(const FBoxBuilderData& InOutBuilderData) override;
protected:
	uint32 NumEmptyBytes = 0;
};


class FBoxMDAT : public FBoxBase
{
public:
	FBoxMDAT() : FBoxBase(BoxAtom('m','d','a','t'))
	{ }
	virtual ~FBoxMDAT() = default;
protected:
};


class FBoxMOOV : public FBoxBase
{
public:
	FBoxMOOV() : FBoxBase(BoxAtom('m','o','o','v'))
	{ }
	virtual ~FBoxMOOV() = default;
protected:
};


class FBoxMVHD : public FBoxFull
{
public:
	FBoxMVHD() : FBoxFull(BoxAtom('m','v','h','d'))
	{ }
	virtual ~FBoxMVHD() = default;

	void SetCreationTime(uint64 InCreationTime)
	{ CreationTime = InCreationTime; }
	void SetModificationTime(uint64 InModificationTime)
	{ ModificationTime = InModificationTime; }
	void SetTimescale(uint32 InTimescale)
	{ Timescale = InTimescale; }
	uint32 GetTimescale() const
	{ return Timescale; }
	void SetDuration(uint64 InDuration)
	{ Duration = InDuration; }
	uint64 GetDuration() const
	{ return Duration; }
	void SetNextTrackID(uint32 InNextTrackID)
	{ NextTrackID = InNextTrackID; }

	bool BuildData(const FBoxBuilderData& InOutBuilderData) override;
	bool UpdateTimescale(uint32 InTimescale);
	bool UpdateDuration(uint64 InDuration);
	bool RewriteChanges(FRewriteBoxChangeDelegate InRewriteBoxChangeDelegate) override;
protected:
	uint64 CreationTime = 0;
	uint64 ModificationTime = 0;
	uint64 Duration = 0;
	uint32 Timescale = 0;
	uint32 NextTrackID = 0;
	int64 TimescalePatchOffset = -1;
	int64 DurationPatchOffset = -1;
};


class FBoxTRAK : public FBoxBase
{
public:
	FBoxTRAK() : FBoxBase(BoxAtom('t','r','a','k'))
	{ }
	virtual ~FBoxTRAK() = default;
protected:
};


class FBoxTKHD : public FBoxFull
{
public:
	FBoxTKHD() : FBoxFull(BoxAtom('t','k','h','d'))
	{ }
	virtual ~FBoxTKHD() = default;

	void SetCreationTime(uint64 InCreationTime)
	{ CreationTime = InCreationTime; }
	void SetModificationTime(uint64 InModificationTime)
	{ ModificationTime = InModificationTime; }
	void SetDuration(uint64 InDuration)
	{ Duration = InDuration; }
	void SetTrackID(uint32 InTrackID)
	{ TrackID = InTrackID; }
	void SetTrackEnabled(bool bIsEnabled)
	{ if (bIsEnabled) Flags |= 1U; else Flags &= ~(1U); }
	void SetTrackInMovie(bool bIsInMovie)
	{ if (bIsInMovie) Flags |= 2U; else Flags &= ~(2U); }
	void SetTrackInPreview(bool bIsInPreview)
	{ if (bIsInPreview) Flags |= 4U; else Flags &= ~(4U); }
	void SetWidth(uint32 InWidth)
	{ Width = InWidth << 16; }
	void SetHeight(uint32 InHeight)
	{ Height = InHeight << 16; }
	void SetVolume(uint8 InVolume)
	{ Volume = (uint16)InVolume << 8; }
	void SetWidthRaw(uint32 InWidth)
	{ Width = InWidth; }
	void SetHeightRaw(uint32 InHeight)
	{ Height = InHeight; }
	void SetVolumeRaw(int16 InVolume)
	{ Volume = (uint16)InVolume; }
	void SetLayer(uint16 InLayer)
	{ Layer = InLayer; }
	void SetAlternateGroup(uint16 InAlternateGroup)
	{ AlternateGroup = InAlternateGroup; }
	void SetMatrix(const int32 InMatrix[9])
	{ FMemory::Memcpy(Matrix, InMatrix, sizeof(Matrix)); }
	void SetCustomFlags(uint32 InFlags)
	{ CustomFlags = InFlags & ~3U; }

	bool BuildData(const FBoxBuilderData& InOutBuilderData) override;

	bool UpdateDuration(uint64 InDuration);
	bool RewriteChanges(FRewriteBoxChangeDelegate InRewriteBoxChangeDelegate) override;
protected:
	int32 Matrix[9] { 0x00010000,0,0,0,0x00010000,0,0,0,0x40000000 };
	uint64 CreationTime = 0;
	uint64 ModificationTime = 0;
	uint64 Duration = 0;
	uint32 CustomFlags = 0;
	uint32 TrackID = 0;
	uint32 Width = 0;
	uint32 Height = 0;
	uint16 Volume = 0;
	uint16 Layer = 0;
	uint16 AlternateGroup = 0;
	int64 DurationPatchOffset = -1;
};


class FBoxMDIA : public FBoxBase
{
public:
	FBoxMDIA() : FBoxBase(BoxAtom('m','d','i','a'))
	{ }
	virtual ~FBoxMDIA() = default;
protected:
};


class FBoxMDHD : public FBoxFull
{
public:
	FBoxMDHD() : FBoxFull(BoxAtom('m','d','h','d'))
	{ }
	virtual ~FBoxMDHD() = default;

	void SetCreationTime(uint64 InCreationTime)
	{ CreationTime = InCreationTime; }
	void SetModificationTime(uint64 InModificationTime)
	{ ModificationTime = InModificationTime; }
	void SetDuration(uint64 InDuration)
	{ Duration = InDuration; }
	void SetTimescale(uint32 InTimescale)
	{ Timescale = InTimescale; }
	void SetLanguage(uint8 In1, uint8 In2, uint8 In3)
	{
		check(In1 >= 0x60 && In1 < 0x80 && In2 >= 0x60 && In2 < 0x80 && In3 >= 0x60 && In3 < 0x80);
		LanguageCode[0] = In1 - 0x60;
		LanguageCode[1] = In2 - 0x60;
		LanguageCode[2] = In3 - 0x60;
	}

	bool BuildData(const FBoxBuilderData& InOutBuilderData) override;
	bool UpdateDuration(uint64 InDuration);
	bool RewriteChanges(FRewriteBoxChangeDelegate InRewriteBoxChangeDelegate) override;
protected:
	uint64 CreationTime = 0;
	uint64 ModificationTime = 0;
	uint64 Duration = 0;
	uint32 Timescale = 0;
	uint8 LanguageCode[3] {};
	int64 DurationPatchOffset = -1;
};


class FBoxHDLR : public FBoxFull
{
public:
	FBoxHDLR() : FBoxFull(BoxAtom('h','d','l','r'))
	{ }
	virtual ~FBoxHDLR() = default;

	void SetHandlerType(uint32 InHandlerType)
	{ HandlerType = InHandlerType; }
	void SetName(const FString& InName)
	{ Name = InName; }

	bool BuildData(const FBoxBuilderData& InOutBuilderData) override;
protected:
	FString Name;
	uint32 HandlerType = 0;
};


class FBoxMINF : public FBoxBase
{
public:
	FBoxMINF() : FBoxBase(BoxAtom('m','i','n','f'))
	{ }
	virtual ~FBoxMINF() = default;
protected:
};


class FBoxVMHD : public FBoxFull
{
public:
	FBoxVMHD() : FBoxFull(BoxAtom('v','m','h','d'))
	{ }
	virtual ~FBoxVMHD() = default;

	bool BuildData(const FBoxBuilderData& InOutBuilderData) override;
protected:
	uint16 GraphicsMode = 0;
	uint16 OpColor[3] { };
};


class FBoxSMHD : public FBoxFull
{
public:
	FBoxSMHD() : FBoxFull(BoxAtom('s','m','h','d'))
	{ }
	virtual ~FBoxSMHD() = default;

	bool BuildData(const FBoxBuilderData& InOutBuilderData) override;
protected:
	uint16 Balance = 0;
};


class FBoxNMHD : public FBoxFull
{
public:
	FBoxNMHD() : FBoxFull(BoxAtom('n','m','h','d'))
	{ }
	virtual ~FBoxNMHD() = default;
protected:
};


class FBoxDINF : public FBoxBase
{
public:
	FBoxDINF() : FBoxBase(BoxAtom('d','i','n','f'))
	{ }
	virtual ~FBoxDINF() = default;
protected:
};


class FBoxDREF : public FBoxFull
{
public:
	FBoxDREF() : FBoxFull(BoxAtom('d','r','e','f'))
	{ }
	virtual ~FBoxDREF() = default;
	bool BuildData(const FBoxBuilderData& InOutBuilderData) override;
protected:
};


class FBoxURL : public FBoxFull
{
public:
	FBoxURL() : FBoxFull(BoxAtom('u','r','l',' '))
	{ }
	virtual ~FBoxURL() = default;
	bool BuildData(const FBoxBuilderData& InOutBuilderData) override;
protected:
};


class FBoxSTBL : public FBoxBase
{
public:
	FBoxSTBL() : FBoxBase(BoxAtom('s','t','b','l'))
	{ }
	virtual ~FBoxSTBL() = default;
protected:
};


class FBoxSampleEntry : public FBoxBase
{
public:
	FBoxSampleEntry(uint32 InFormat) : FBoxBase(InFormat)
	{ }
	virtual ~FBoxSampleEntry() = default;
	void SetAdditionalBoxes(const TArray<TArray<uint8>>& InAdditionalBoxes)
	{ AdditionalBoxes = InAdditionalBoxes; }
	bool BuildData(const FBoxBuilderData& InOutBuilderData) override;
protected:
	bool AppendAdditionalBoxes();
	TArray<TArray<uint8>> AdditionalBoxes;
	uint16 DataReferenceIndex = 1;
};


class FBoxVisualSampleEntry : public FBoxSampleEntry
{
public:
	FBoxVisualSampleEntry(uint32 InFormat) : FBoxSampleEntry(InFormat)
	{ }
	virtual ~FBoxVisualSampleEntry() = default;
	void SetWidth(uint16 InWidth)
	{ Width = InWidth; }
	void SetHeight(uint16 InHeight)
	{ Height = InHeight; }
	void SetCompressorName(const FString& InCompressorName)
	{ CompressorName = InCompressorName; }
	bool BuildData(const FBoxBuilderData& InOutBuilderData) override;
protected:
	FString CompressorName;
	uint16 Width = 0;
	uint16 Height = 0;
};


class FBoxAudioSampleEntry : public FBoxSampleEntry
{
public:
	FBoxAudioSampleEntry(uint32 InFormat) : FBoxSampleEntry(InFormat)
	{ }
	virtual ~FBoxAudioSampleEntry() = default;
	void SetChannelCount(uint16 InChannelCount)
	{ ChannelCount = InChannelCount; }
	bool SetSampleRate(uint32 InSampleRate)
	{ SampleRate = InSampleRate; return SampleRate >= 0x10000; }
	bool BuildData(const FBoxBuilderData& InOutBuilderData) override;
protected:
	uint32 SampleRate = 0;
	uint16 ChannelCount = 0;
};


class FBoxTimecodeSampleEntry : public FBoxSampleEntry
{
public:
	FBoxTimecodeSampleEntry(uint32 InFormat) : FBoxSampleEntry(InFormat)
	{ }
	virtual ~FBoxTimecodeSampleEntry() = default;

	void SetTimescale(int32 InTimescale)
	{ Timescale = InTimescale; }
	void SetFrameDuration(int32 InFrameDuration)
	{ FrameDuration = InFrameDuration; }
	void SetFramesPerSecond(int8 InFramesPerSecond)
	{ FramesPerSecond = InFramesPerSecond; }
	void SetDropFrame(bool bInDropFrame)
	{ bDropFrame = bInDropFrame; }
	void SetMax24Hours(bool bInMax24h)
	{ bMax24Hours = bInMax24h; }
	void SetAllowNegativeTimes(bool bInAllowNegativeTimes)
	{ bAllowNegativeTimes = bInAllowNegativeTimes; }

	bool BuildData(const FBoxBuilderData& InOutBuilderData) override;
protected:
	// These values are defined to be signed integers!
	int32 Timescale = 0;
	int32 FrameDuration = 0;
	int8 FramesPerSecond = 0;

	bool bDropFrame = false;
	bool bMax24Hours = false;
	bool bAllowNegativeTimes = false;
};


class FBoxSTSD : public FBoxFull
{
public:
	FBoxSTSD() : FBoxFull(BoxAtom('s','t','s','d'))
	{ }
	virtual ~FBoxSTSD() = default;
	void SetVersion(uint8 InVersion)
	{ Version = InVersion; }
	bool BuildData(const FBoxBuilderData& InOutBuilderData) override;
protected:
};


class FBoxSTTS : public FBoxFull
{
public:
	FBoxSTTS() : FBoxFull(BoxAtom('s','t','t','s'))
	{ }
	virtual ~FBoxSTTS() = default;
	void AddSampleDuration(uint32 InSampleDuration)
	{
		const uint32 Dur = InSampleDuration;
		if (SampleDurations.IsEmpty() || SampleDurations[SampleDurations.Num() - 1].Delta != Dur)
		{
			FEntry e;
			e.Count = 1;
			e.Delta = Dur;
			SampleDurations.Emplace(e);
		}
		else
		{
			++SampleDurations[SampleDurations.Num() - 1].Count;
		}
	}
	bool BuildData(const FBoxBuilderData& InOutBuilderData) override;
protected:
	struct FEntry
	{
		uint32 Count;
		uint32 Delta;
	};
	TChunkedArray<FEntry, 64> SampleDurations;
};


class FBoxSTSS : public FBoxFull
{
public:
	FBoxSTSS() : FBoxFull(BoxAtom('s','t','s','s'))
	{ }
	virtual ~FBoxSTSS() = default;
	void AddSampleIndex(uint32 InSampleNumber)
	{ SyncSampleIndices.Emplace(InSampleNumber); }
	bool BuildData(const FBoxBuilderData& InOutBuilderData) override;
protected:
	TChunkedArray<uint32, 240> SyncSampleIndices;
};


class FBoxCTTS : public FBoxFull
{
public:
	FBoxCTTS() : FBoxFull(BoxAtom('c','t','t','s'))
	{ }
	virtual ~FBoxCTTS() = default;
	void AddSampleCompositionOffset(int32 InCompositionTimeOffset)
	{
		if (CompositionTimeOffsets.IsEmpty() || CompositionTimeOffsets[CompositionTimeOffsets.Num() - 1].Delta != InCompositionTimeOffset)
		{
			FEntry e;
			e.Count = 1;
			e.Delta = InCompositionTimeOffset;
			CompositionTimeOffsets.Emplace(e);
		}
		else
		{
			++CompositionTimeOffsets[CompositionTimeOffsets.Num() - 1].Count;
		}
		if (!bHasNegativeEntries && InCompositionTimeOffset < 0)
		{
			bHasNegativeEntries = true;
		}
	}
	bool HaveCompositionOffsets() const
	{ return !(CompositionTimeOffsets.IsEmpty() || (CompositionTimeOffsets.Num() == 1 && CompositionTimeOffsets[0].Delta == 0)); }
	bool HasNegativeCompositionOffsets() const
	{ return bHasNegativeEntries; }
	bool BuildData(const FBoxBuilderData& InOutBuilderData) override;
protected:
	struct FEntry
	{
		uint32 Count;
		int32 Delta;
	};
	TChunkedArray<FEntry, 64> CompositionTimeOffsets;
	bool bHasNegativeEntries = false;
};


class FBoxSTSC : public FBoxFull
{
public:
	FBoxSTSC() : FBoxFull(BoxAtom('s','t','s','c'))
	{ }
	virtual ~FBoxSTSC() = default;
	void AddEntry(uint32 InFirstChunk, uint32 InSamplesPerChunk, uint32 InSampleDescriptionIndex)
	{
		if (SampleToChunkEntries.IsEmpty() ||
			SampleToChunkEntries[SampleToChunkEntries.Num() - 1].samples_per_chunk != InSamplesPerChunk ||
			SampleToChunkEntries[SampleToChunkEntries.Num() - 1].sample_description_index != InSampleDescriptionIndex)
		{
			FEntry e;
			e.first_chunk = InFirstChunk;
			e.samples_per_chunk = InSamplesPerChunk;
			e.sample_description_index = InSampleDescriptionIndex;
			SampleToChunkEntries.Emplace(e);
		}
	}
	bool BuildData(const FBoxBuilderData& InOutBuilderData) override;
protected:
	struct FEntry
	{
		uint32 first_chunk = 0;
		uint32 samples_per_chunk = 0;
		uint32 sample_description_index = 0;
	};
	TChunkedArray<FEntry, 32> SampleToChunkEntries;
};


class FBoxSTSZ : public FBoxFull
{
public:
	FBoxSTSZ() : FBoxFull(BoxAtom('s','t','s','z'))
	{ }
	virtual ~FBoxSTSZ() = default;
	void SetConstantSampleSize(uint32 InConstantSampleSize)
	{ ConstantSampleSize = InConstantSampleSize; }
	void AddSampleSize(uint32 InSampleSize)
	{
		bHasVaryingSizes = bHasVaryingSizes || (!SampleSizes.IsEmpty() && SampleSizes[SampleSizes.Num()-1] != InSampleSize);
		SampleSizes.Emplace(InSampleSize);
		LargestSampleSize = InSampleSize > LargestSampleSize ? InSampleSize : LargestSampleSize;
	}
	uint32 GetNumberOfSamples() const
	{ return SampleSizes.Num(); }
	bool BuildData(const FBoxBuilderData& InOutBuilderData) override;
protected:
	TChunkedArray<uint32, 240> SampleSizes;
	uint32 ConstantSampleSize = 0;
	uint32 LargestSampleSize = 0;
	bool bHasVaryingSizes = false;
	bool bUseSmallSizeBox = false;
};


class FBoxSTCO : public FBoxFull
{
public:
	FBoxSTCO() : FBoxFull(BoxAtom('s','t','c','o'))
	{ }
	virtual ~FBoxSTCO() = default;
	void AddChunkOffset(int64 InChunkOffset)
	{
		ChunkOffsets.Emplace(InChunkOffset);
		if (InChunkOffset >= 0x100000000LL)
		{
			bNeeds64Bits = true;
		}
	}
	uint32 GetNumberOfChunkOffsets() const
	{ return ChunkOffsets.Num(); }
	const TChunkedArray<int64, 32>& GetChunkOffsets() const
	{ return ChunkOffsets; }
	bool Needs64Bits() const
	{ return bNeeds64Bits; }
	void Force64Bits()
	{ bNeeds64Bits = true; }
	bool AddFileOffset(int64 InOffsetToAdd);
	bool BuildData(const FBoxBuilderData& InOutBuilderData) override;
protected:
	TChunkedArray<int64, 32> ChunkOffsets;
	bool bNeeds64Bits = false;
};


class FBoxMVEX : public FBoxBase
{
public:
	FBoxMVEX() : FBoxBase(BoxAtom('m','v','e','x'))
	{ }
	virtual ~FBoxMVEX() = default;
protected:
};


class FBoxMEHD : public FBoxFull
{
public:
	FBoxMEHD() : FBoxFull(BoxAtom('m','e','h','d'))
	{ }
	virtual ~FBoxMEHD() = default;
	void SetFragmentDuration(uint64 InFragmentDuration)
	{ FragmentDuration = InFragmentDuration; }
	bool BuildData(const FBoxBuilderData& InOutBuilderData) override;
	bool UpdateFragmentDuration(uint64 InFragmentDuration);
	bool RewriteChanges(FRewriteBoxChangeDelegate InRewriteBoxChangeDelegate) override;
protected:
	uint64 FragmentDuration = 0;
	int64 FragmentDurationPatchOffset = -1;
};


class FBoxTREX : public FBoxFull
{
public:
	FBoxTREX() : FBoxFull(BoxAtom('t','r','e','x'))
	{ }
	virtual ~FBoxTREX() = default;
	void SetTrackID(uint32 InTrackID)
	{ TrackID = InTrackID; }
	void SetDefaultSampleDuration(uint32 InDefaultSampleDuration)
	{ default_sample_duration = InDefaultSampleDuration; }
	void SetDefaultSampleSize(uint32 InDefaultSampleSize)
	{ default_sample_size = InDefaultSampleSize; }
	void SetDefaultSampleFlags(uint32 InDefaultSampleFlags)
	{ default_sample_flags = InDefaultSampleFlags; }
	uint32 GetDefaultSampleFlags() const
	{ return default_sample_flags; }
	bool BuildData(const FBoxBuilderData& InOutBuilderData) override;
protected:
	uint32 TrackID = 0;
	uint32 default_sample_description_index = 1;
	uint32 default_sample_duration = 0;
	uint32 default_sample_size = 0;
	uint32 default_sample_flags = 0;
};


class FBoxMOOF : public FBoxBase
{
public:
	FBoxMOOF() : FBoxBase(BoxAtom('m','o','o','f'))
	{ }
	virtual ~FBoxMOOF() = default;
protected:
};


class FBoxMFHD : public FBoxFull
{
public:
	FBoxMFHD() : FBoxFull(BoxAtom('m','f','h','d'))
	{ }
	virtual ~FBoxMFHD() = default;
	void SetSequenceNumber(uint32 InSequenceNumber)
	{ SequenceNumber = InSequenceNumber; }
	bool BuildData(const FBoxBuilderData& InOutBuilderData) override;
protected:
	uint32 SequenceNumber = 0;
};


class FBoxTRAF : public FBoxBase
{
public:
	FBoxTRAF() : FBoxBase(BoxAtom('t','r','a','f'))
	{ }
	virtual ~FBoxTRAF() = default;
protected:
};


class FBoxTFHD : public FBoxFull
{
public:
	FBoxTFHD() : FBoxFull(BoxAtom('t','f','h','d'))
	{ }
	virtual ~FBoxTFHD() = default;
	void SetTrackID(uint32 InTrackID)
	{ TrackID = InTrackID; }
	void SetDefaultBaseIsMoof(bool bInDefaultBaseIsMoof)
	{ bDefaultBaseIsMoof = bInDefaultBaseIsMoof; }
	void SetBaseDataOffset(uint64 InBaseDataOffset)
	{ base_data_offset = InBaseDataOffset; }
	void SetSampleDescriptionIndex(uint32 InSampleDescriptionIndex)
	{ sample_description_index = InSampleDescriptionIndex; }
	void SetDefaultSampleDuration(uint32 InDefaultSampleDuration)
	{ default_sample_duration = InDefaultSampleDuration; }
	void SetDefaultSampleSize(uint32 InDefaultSampleSize)
	{ default_sample_size = InDefaultSampleSize; }
	void SetDefaultSampleFlags(uint32 InDefaultSampleFlags)
	{ default_sample_flags = InDefaultSampleFlags; }
	TOptional<uint32> GetDefaultSampleFlags() const
	{ return default_sample_flags; }

	bool BuildData(const FBoxBuilderData& InOutBuilderData) override;
	bool UpdateBaseDataOffset(uint64 InBaseDataOffset);
	bool RewriteChanges(FRewriteBoxChangeDelegate InRewriteBoxChangeDelegate) override;
protected:
	uint32 TrackID = 0;
	bool bDefaultBaseIsMoof = false;
	TOptional<uint64> base_data_offset;
	TOptional<uint32> sample_description_index;
	TOptional<uint32> default_sample_duration;
	TOptional<uint32> default_sample_size;
	TOptional<uint32> default_sample_flags;
	int64 BaseDataPatchOffset = -1;
};


class FBoxTFDT : public FBoxFull
{
public:
	FBoxTFDT() : FBoxFull(BoxAtom('t','f','d','t'))
	{ }
	virtual ~FBoxTFDT() = default;

	void SetBaseMediaDecodeTime(uint64 InBaseMediaDecodeTime)
	{ BaseMediaDecodeTime = InBaseMediaDecodeTime; }

	bool BuildData(const FBoxBuilderData& InOutBuilderData) override;
protected:
	uint64 BaseMediaDecodeTime = 0;
};


class FBoxTRUN : public FBoxFull
{
public:
	FBoxTRUN() : FBoxFull(BoxAtom('t','r','u','n'))
	{ }
	virtual ~FBoxTRUN() = default;

	void SetDataOffset(int32 InDataOffset)
	{ DataOffset = InDataOffset; }
	// Set default flags as they appear in the `trex` or `tfhd` box.
	// The `trun` has no default flags and we use this value only as a convenience for testing.
	void SetDefaultSampleFlagsForTesting(uint32 InDefaultSampleFlags)
	{ DefaultSampleFlags = InDefaultSampleFlags; }
	void SetFirstSampleFlags(uint32 InFirstSampleFlags)
	{ first_sample_flags = InFirstSampleFlags; }
	void AddSample(uint32 InSampleDuration, uint32 InSampleSize, uint32 InSampleFlags, int32 InSampleCompositionTimeOffset)
	{
		FEntry e;
		e.sample_duration = InSampleDuration;
		e.sample_size = InSampleSize;
		e.sample_flags = InSampleFlags;
		if ((e.sample_composition_time_offset = InSampleCompositionTimeOffset) != 0)
		{
			bRequiresCompositionTimeOffsets = true;
		}
		if (Samples.Num())
		{
			if (Samples[Samples.Num()-1].sample_size != e.sample_size)
			{
				bHasDifferentSizes = true;
			}
			if (Samples[Samples.Num()-1].sample_duration != e.sample_duration)
			{
				bHasDifferentDurations = true;
			}
			if (Samples[Samples.Num()-1].sample_flags != e.sample_flags)
			{
				// Do samples starting from the 2nd onward have different flags?
				bFlagsAfterFirstAreDifferent = Samples.Num() > 1;
			}
		}
		Samples.Emplace(MoveTemp(e));
	}

	uint32 GetFirstSampleFlagsOrDefault() const
	{ return Samples.Num() ? Samples[0].sample_flags : DefaultSampleFlags; }
	uint32 GetSubsequentSampleFlagsOrDefault() const
	{ return Samples.Num() > 1 ? Samples[1].sample_flags : DefaultSampleFlags; }
	bool AreFlagsAfterFirstSampleDifferent() const
	{ return bFlagsAfterFirstAreDifferent; }

	bool RequiresCompositionTimeOffsets() const
	{ return bRequiresCompositionTimeOffsets; }
	bool HasDifferentSizes() const
	{ return bHasDifferentSizes; }
	bool HasDifferentDurations() const
	{ return bHasDifferentDurations; }
	void DoNotWriteDurations()
	{ bDoNotWriteDurations = true; }
	void DoNotWriteSizes()
	{ bDoNotWriteSizes = true; }
	void DoNotWriteFlags()
	{ bDoNotWriteFlags = true; }

	bool BuildData(const FBoxBuilderData& InOutBuilderData) override;
protected:
	struct FEntry
	{
		uint32 sample_duration = 0;
		uint32 sample_size = 0;
		uint32 sample_flags = 0;
		int32 sample_composition_time_offset = 0;
	};
	TChunkedArray<FEntry, 64> Samples;
	TOptional<int32> DataOffset;
	TOptional<uint32> first_sample_flags;
	uint32 DefaultSampleFlags = 0;
	bool bRequiresCompositionTimeOffsets = false;
	bool bHasDifferentSizes = false;
	bool bHasDifferentDurations = false;
	bool bFlagsAfterFirstAreDifferent = false;
	bool bDoNotWriteDurations = false;
	bool bDoNotWriteSizes = false;
	bool bDoNotWriteFlags = false;
	int64 DataOffsetPatchOffset = -1;
};


class FBoxSUBS : public FBoxFull
{
public:
	FBoxSUBS() : FBoxFull(BoxAtom('s','u','b','s'))
	{ }
	virtual ~FBoxSUBS() = default;

	using FSubSampleInfo = IMP4RawMuxer::FTrackSample::FSubSampleInfo;

	void AddSubsamplesForSample(uint32 InSampleNumber, const TArray<FSubSampleInfo>& InSubsampleInfo)
	{
		check(InSampleNumber);	// samples start counting at 1, there is no 0.
		// No need to record samples with no subsample information.
		if (InSubsampleInfo.Num())
		{
			SubSamples.Emplace(FSSI({InSampleNumber, InSubsampleInfo}));
		}
	}

	bool BuildData(const FBoxBuilderData& InOutBuilderData) override;
protected:
	struct FSSI
	{
		uint32 SampleNumber = 0;
		TArray<FSubSampleInfo> Subs;
	};
	TChunkedArray<FSSI, 32> SubSamples;
};


class FBoxEDTS : public FBoxBase
{
public:
	FBoxEDTS() : FBoxBase(BoxAtom('e','d','t','s'))
	{ }
	virtual ~FBoxEDTS() = default;
protected:
};


class FBoxELST : public FBoxFull
{
public:
	FBoxELST() : FBoxFull(BoxAtom('e','l','s','t'))
	{ }
	virtual ~FBoxELST() = default;
	void SetEditDuration(uint64 InEditDuration)
	{ EditDuration = InEditDuration; }
	void SetMediaTime(int64 InMediaTime)
	{ MediaTime = InMediaTime; }
	bool UpdateEditDuration(uint64 InEditDuration);
	bool RewriteChanges(FRewriteBoxChangeDelegate InRewriteBoxChangeDelegate) override;
	bool BuildData(const FBoxBuilderData& InOutBuilderData) override;
protected:
	uint64 EditDuration = 0;
	int64 MediaTime = 0;
	int64 EditDurationPatchOffset = -1;
};


class FBoxUDTA : public FBoxBase
{
public:
	FBoxUDTA() : FBoxBase(BoxAtom('u','d','t','a'))
	{ }
	virtual ~FBoxUDTA() = default;
protected:
};


class FBoxNAMEbase : public FBoxBase
{
public:
	FBoxNAMEbase() : FBoxBase(BoxAtom('n','a','m','e'))
	{ }
	virtual ~FBoxNAMEbase() = default;

	void SetName(const FString& InName)
	{ Name = InName; }

	bool BuildData(const FBoxBuilderData& InOutBuilderData) override;
protected:
	FString Name;
};

class FBoxNAME : public FBoxFull
{
public:
	FBoxNAME() : FBoxFull(BoxAtom('n','a','m','e'))
	{ }
	virtual ~FBoxNAME() = default;

	void SetName(const FString& InName)
	{ Name = InName; }

	bool BuildData(const FBoxBuilderData& InOutBuilderData) override;
protected:
	FString Name;
};


class FBoxTREF : public FBoxBase
{
public:
	FBoxTREF() : FBoxBase(BoxAtom('t','r','e','f'))
	{ }
	virtual ~FBoxTREF() = default;
protected:
};

class FBoxTREFType : public FBoxBase
{
public:
	FBoxTREFType() : FBoxBase(0)
	{ }
	virtual ~FBoxTREFType() = default;

	void SetReferenceType(uint32 InReferenceType)
	{ ReferenceType = InReferenceType; }
	void AddReferencedTrackNumber(uint32 InReferencedTrackNumber)
	{ ReferencedTrackNumbers.Emplace(InReferencedTrackNumber); }
	bool BuildData(const FBoxBuilderData& InOutBuilderData) override;
protected:
	uint32 ReferenceType = 0;
	TArray<uint32> ReferencedTrackNumbers;
};


class FBoxOpaqueData : public FBoxBase
{
public:
	FBoxOpaqueData(uint32 InType) : FBoxBase(InType)
	{ }
	virtual ~FBoxOpaqueData() = default;

	void SetRawBoxData(TConstArrayView<uint8> InRawBoxData)
	{ RawBoxData = InRawBoxData; }
	bool BuildData(const FBoxBuilderData& InOutBuilderData) override;
protected:
	TArray<uint8> RawBoxData;
};

}

