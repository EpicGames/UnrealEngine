// Copyright Epic Games, Inc. All Rights Reserved.

#include "MP4MuxerModule.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "MP4MuxerModule"

DEFINE_LOG_CATEGORY(LogMP4Muxer);

class FMP4MuxerModule : public IModuleInterface
{
public:
	void StartupModule() override
	{
	}

	void ShutdownModule() override
	{
	}

	bool SupportsDynamicReloading() override
	{
		return false;
	}

private:
};

IMPLEMENT_MODULE(FMP4MuxerModule, MP4Muxer);

#undef LOCTEXT_NAMESPACE
