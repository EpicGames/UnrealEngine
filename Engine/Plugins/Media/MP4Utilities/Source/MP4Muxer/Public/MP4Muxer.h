// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "Misc/Timespan.h"
#include "Misc/FrameRate.h"
#include "Misc/TVariant.h"
#include "Misc/Optional.h"


#define UE_API MP4MUXER_API

/**
 * This module implements a basic mp4 file multiplexer/writer.
 *
 * The data to be written is handled as RAW data and is not checked for validity.
 * Data is also expected to be framed already, that is, one data packet corresponds
 * to one media sample.
 *
 * If the packet data requires internal sample sizes, eg. NAL unit start codes to
 * be replaced with the NAL size for AVC/HEVC video, this needs to have been done
 * already as well.
 *
 * Codec specific data, like configuration boxes, also need to be set up already
 * in an mp4 box structure and provided per box, which will also just be
 * written as raw data.
 */
class IMP4RawMuxer
{
public:
	virtual ~IMP4RawMuxer() = default;

	UE_API static TSharedRef<IMP4RawMuxer, ESPMode::ThreadSafe> Create();

	// Wraps binary data in a box. Useful for creating the additional FTrackSpec::SampleEntryBoxes
	UE_API static TArray<uint8> WrapDataInBox(uint32 InBoxAtom, const TConstArrayView<uint8> InBoxData);

	// Wraps binary data in a full box. Useful for creating the additional FTrackSpec::SampleEntryBoxes
	UE_API static TArray<uint8> WrapDataInBox(uint32 InBoxAtom, uint8 InBoxVersion, uint32 InBoxFlags, const TConstArrayView<uint8> InBoxData);

	struct FTrackSpec
	{
		struct FVideo
		{
			uint32 DisplayWidth = 0;
			uint32 DisplayHeight = 0;
			FString CompressorName;
		};
		struct FAudio
		{
			uint32 SamplingRate = 0;
			uint16 NumberOfChannels = 0;
		};
		struct FTimecode
		{
			/*
				Timecode is always written as `tmcd` so this are its parameters
				See https://developer.apple.com/documentation/quicktime-file-format/timecode_sample_description
				Please note that the "tape counter" flag is no longer mentioned in https://developer.apple.com/library/archive/technotes/tn2310/_index.html
				and the technote seems to suggest that only frame counter values are written.
				Also see https://developer.apple.com/documentation/coremedia/time-code-flags?changes=_5_3&language=objc
				As such the deprecated "counter" flag (value was 0x8) will not be written.

				You can write timecode tracks with as many samples as the associated video track or write
				only single timecode sample that is associated with the first video frame.
				This is what most writing programs do since the timecode of subsequent frames simply
				follows from the first and videos usually do not have discontinuties.
				When writing multiple video tracks and their associated timecode tracks you need to add
				a reference to the timecode track to the video by calling `AddTrackReference()`.
				If there is only a single timecode track that applies to all video tracks (or there is
				only one video track) you should still associate the tracks but this is not strictly
				necessary.
			*/
			bool bDropFrame = false;
			bool bMax24Hours = false;
			bool bAllowNegativeTimes = false;
			// These values are defined to be signed integers!
			int32 Timescale = 0;
			int32 FrameDuration = 0;
			int8 FramesPerSecond = 0;
		};
		struct FOpaqueData
		{
			/*
				Arbitrary data can be written into the file as long as it conforms to the general
				ISO/IEC 14496-12 sample structure.

				This is useful if you want to add subtitle data where each of the formats like
				`tx3g`, `wvtt` or `stpp` all use different handlers, media headers and data layout.
				You can write any data of course provided that it does not contain offsets into
				your data that depend on the position of any data written to the file, because
				offsets cannot be adjusted by this writer for opaque data.

				Since this writer only writes raw data in the first place, constructing the
				boxes for any opaque data falls onto your application as well.
			*/

			// Values to put into the `tkhd` box.
			int32 matrix[9] { 0x00010000,0,0,0,0x00010000,0,0,0,0x40000000 };
			uint32 width = 0;
			uint32 height = 0;
			uint32 flags = 0;	// the track's `bEnabled` and `bInMovie` will overwrite what is set here.
			int16 volume = 0;


			// Type of handler to write to the `mdia`/`hdlr` box.
			uint32 mdia_hdlr_Type = 0;
			// Name of the handler to write to the `mdia`/`hdlr` box.
			FString mdia_hdlr_Name;

			// Type of media header box to write into the `minf` box (eg `nmhd`)
			uint32 minf_MediaHeader_Type = 0;
			// Raw data of the media header box (excluding the box size and box type)
			// to write into the `minf` box.
			// If this box is a full box, then the raw data needs to include the version and flags
			// right before the payload.
			TArray<uint8> minf_MediaHeader_RawBoxData;

			// Raw `SampleEntry` (ISO/IEC 14496-12:2022 - 8.5.2.2 Syntax) data
			// (excluding the box size and box type) to write into the `stsd` box.
			// This sample entry data MUST include the base `SampleEntry` data members!
			// The sample entry type is given by the track's `SampleEntryFormat` parameter (see below).
			TArray<uint8> stsd_SampleEntry_RawBoxData;
		};

		/**
		 * ISO/IEC 14496-12 Section 8.5.2.1 "format"
		 */
		uint32 SampleEntryFormat = 0;

		/**
		 * Set to `true` if every sample in this track is independently
		 * decodable and is a sync sample (keyframe).
		 * Set to `false` if samples depend on other samples.
		 */
		bool bIsAllKeyframes = true;

		/**
		 * The number of ticks per second in which this tracks sample
		 * decode and presentation time as well as the sample duration
		 * is measured in.
		 * Must not be zero.
		 */
		uint32 Timescale = 0;

		/**
		 * If every sample has the same duration you can set this fixed
		 * value here for convenience and then specify 0 for duration in
		 * the FTrackSample structure.
		 */
		uint32 ConstantSampleDuration = 0;

		/**
		 * Track type specific values that need to be set.
		 * The type this variant contains determines if the track contains
		 * video, audio, timecode or other.
		 * The values in these structures are written as-is and should
		 * therefore be set correctly.
		 */
		TVariant<FVideo,FAudio,FTimecode,FOpaqueData> Properties;

		/**
		 * Additional pre-built boxes to be added to the sample type
		 * (see `SampleEntryFormat` above) that is written to the `stsd` box.
		 * This should contain codec specific data like the configuration
		 * record and other (optional) boxes.
		 */
		TArray<TArray<uint8>> SampleEntryBoxes;

		/**
		 * If set, a subsample box `subs` will be added to the `stsd` or
		 * `traf` box with subsample information provided for each sample
		 * when adding it to the track.
		 * This value contains the 24 bit flags value for the `subs` box.
		 * When written the `subs` box will always be of version 1 to use
		 * 32 bit subsample sizes.
		 */
		TOptional<uint32> SubSampleFlags;

		/**
		 * Various flags to set in the `tkhd` box.
		 * See ISO/IEC 14496-12:2022 - 8.3.2 Track header box
		 */
		uint16 Layer = 0;
		uint16 AlternateGroup = 0;
		bool bEnabled = true;
		bool bInMovie = true;

		/**
		 * Language tag to set in the `mvhd` box.
		 * Must be a 3 letter ISO-639-2/T code.
		 */
		uint8 Language[3] { 'u','n','d' };

		/**
		 * If set this gets written as an `udta`/`name` box into the track to
		 * give the track a human readable label.
		 */
		FString Name;
	};


	struct FConfiguration
	{
		enum class EMuxMode
		{
			// Standard file, `moov` box at the end.
			Standard,
			// Optimized file, `moov` box in front. Requires a temporary file and a 2nd cleanup pass.
			WebOptimized,
			// Fragmented file
			Fragmented,
		};

		/**
		 * Filename of the file to write.
		 */
		FString OutputFilename;

		/**
		 * Filename for the temporary file in `WebOptimized` mode.
		 * It will be deleted after optimization unless an error occurred
		 * or you have aborted muxing by calling `StopAndClose()`.
		 */
		FString TemporaryFilename;

		/**
		 * Size of the temporary buffer used to read-write data to and from
		 * during the optimization phase in `WebOptimized` mode.
		 * Data of the unoptimized file is read into this buffer and written
		 * into the optimized file from here. The larger the buffer the
		 * fewer read/writes are necessary.
		 * If kept at zero a suitable buffer size is chosen automatically.
		 * This is only allocated in `WebOptimized` mode.
		 */
		uint32 TemporaryOptimizationBufferSize = 0;

		/**
		 * Whether to write a standard mp4 (`moov` box will be placed at the end) or
		 * a fragmented mp4.
		 */
		EMuxMode MuxMode = EMuxMode::Standard;

		/**
		 * Duration for each chunk of sample data.
		 * Samples of each track will be collected until this duration is reached
		 * (or the track has no more samples to write) and then written to disk as
		 * a group of samples per track.
		 *
		 * This value must not less or equal to zero.
		 * A typical value is 500ms. The larger the value the more data the muxer
		 * needs to collect and keep, requiring more memory.
		 */
		FTimespan InterleaveDuration;


		/**
		 * Optional progress delegate to invoke periodically during the optimization
		 * phase in `WebOptimized` mode.
		 * There is no progress delegate for `Standard` or `Fragmented` mode since
		 * the muxer does no know how many samples will be written.
		 * If you want to display progress you can do this during the muxing process
		 * by tracking which sample number (out of however many, which you should know)
		 * is currently being written into the file.
		 */
		DECLARE_DELEGATE_OneParam(FOptimizeProgressDelegate, float /*PrecentComplete*/);
		FOptimizeProgressDelegate OptimizeProgressDelegate;
	};

	/**
	 * Configures the muxer.
	 */
	virtual bool Configure(const FConfiguration& InConfiguration) = 0;

	/**
	 * Adds a track to be written.
	 * Returns a track index used during muxing or -1 if the track
	 * cannot be added.
	 */
	virtual int32 AddTrack(const FTrackSpec& InTrackSpec) = 0;

	/**
	 * Adds a track reference.
	 * This is usually only done for timecode tracks being referenced by one or more video tracks.
	 * See ISO/IEC 14496-12:2022 section 8.3.3 Track reference box
	 * Example:
	 *  If you added a video track (track index 1) and a timecode track (track index 3) and you
	 *  wish the video to reference the timecode track you call
	 *  AddTrackReference(1, 3, MakeBoxAtom('t','m','c','d'))
	 */
	virtual bool AddTrackReference(int32 InTrackIndexToAddReferenceTo, int32 InTrackIndexBeingReferenced, uint32 InReferenceType) = 0;

	/**
	 * State value reported to the registered `FStatusDelegate`.
	 */
	enum class EStatus
	{
		// Failure.
		Failed,
		// Multiplexing completed successfully.
		Finished
	};
	DECLARE_DELEGATE_OneParam(FStatusDelegate, EStatus /*Status*/);

	/**
	 * Defines an entry in the sample request array the registered
	 * `FRequestTrackDataDelegate` provides indicating for which
	 * track(s) another sample is needed before a new chunk
	 * (a group of samples) can be written.
	 */
	struct FSampleRequest
	{
		int32 TrackIndex = -1;
		uint32 SampleNumber = 0;
	};
	DECLARE_DELEGATE_OneParam(FRequestTrackDataDelegate, const TArray<FSampleRequest>& /*TracksWantingSamples*/);

	/**
	 * This method starts the muxing process.
	 *
	 * While there are samples to be muxed:
	 *   The muxer invokes the `FRequestTrackDataDelegate`, providing an array of track indices
	 *   (the value returned by `AddTrack`) and the sample number needed.
	 *   The delegate is invoked from a muxer internal worker thread and is called at any time
	 *   the muxer needs (or still needs) another sample for the particular track.
	 *
	 *   NOTE:
	 *     Due to the asynchronous nature it will happen that the `FRequestTrackDataDelegate`
	 *     asks for the same sample of a track more than once.
	 *     For example, if you want to mux three tracks with the track indices 0, 1 and 2 the
	 *     first invocation of the delegate will ask for sample number 0 of tracks 0, 1 and 2.
	 *     As soon as you provide sample 0 of track 0 by calling `AddTrackSample()` the delegate
	 *     will be invoked again, this time asking for track 0 sample 1 and *still* for track
	 *     1 sample 0 and track 2 sample 0 because these samples have not been received yet.
	 *
	 *     As a result this means that even if you are currently in the process of gathering
	 *     sample 0 of track 1 in another parallel worker thread of your own you must not
	 *     take this new sample request as a new workload. You have to keep track which
	 *     track samples are currently being worked on and ignore any sample request that
	 *     pops up twice (or more).
	 *
	 *   You provide the requested sample by calling `AddTrackSample()`. If this is the last
	 *   sample for this track you need to set the `bIsFinalSample` flag.
	 *   If you do not know that this will be the last sample at that point you can provide
	 *   an empty sample on the next call with that flag set.
	 *
	 *   If during muxing an error occurs the `FStatusDelegate` will be invoked with an
	 *   `EStatus::Failed`. Muxing will stop and you can get a human readable error
	 *   message by calling `GetLastError()`.
	 *   When muxing is complete and the file has been successully written the `FStatusDelegate`
	 *   will be invoked with an `EStatus::Finished`.
	 *
	 *   If you abort muxing by calling `StopAndClose()` the `FStatusDelegate` will not be invoked.
	 *
	 *   If an error occurs the partially written file will be left in place and not be
	 *   deleted. This is true for the temporary file in `WebOptimized` mode as well,
	 *   which is only deleted if the optimization step has finished successfully.
	 */
	virtual bool Start(FRequestTrackDataDelegate InSampleRequestDelegate, FStatusDelegate InStatusDelegate) = 0;


	/**
	 * This structure defines the sample to be added to a track
	 * during the muxing process.
	 * All members should be set correctly as no plausibility checks
	 * are performed except for
	 *  - `Data.Num()` being less than 2^32. A single sample cannot
	 *     be larger than 4 GiB in an ISO/IEC 14496-12:2022 file.
	 */
	struct FTrackSample
	{
		struct FSubSampleInfo
		{
			uint32 codec_specific_parameters = 0;
			uint32 subsample_size = 0;
			uint8 subsample_priority = 0;
			uint8 discardable = 0;
		};
		TArray<FSubSampleInfo> SubSamples;

		/**
		 * The sample data to be written.
		 * Must already be in the final format as it is written verbatim.
		 */
		TConstArrayView64<uint8> Data;

		/**
		 * Presentation duration of this sample, given in the tracks timescale units.
		 * Must not be zero unless a constant duration has been set for the track
		 * which must not be zero.
		 * The value may be zero for the last sample although this is not recommended.
		 * This value is the most important value as it is written to the `stts`
		 * box directly and their relative sum determines the decode time (DTS).
		 */
		uint32 Duration = 0;

		/**
		 * Decode time, given in the tracks timescale units.
		 * This value is NOT written into the file directly.
		 * An ISO/IEC 14496-12:2022 file only stores DTS differences between
		 * samples, implicitly starting at zero.
		 * The first DTS may be used to generate an `elst` box if not zero.
		 * For keyframe-only tracks that do not need a time offset in an
		 * edit list you can leave both DTS and PTS set to zero for every
		 * sample and only specify the sample duration.
		 */
		int64 DTS = 0;

		/**
		 * Presentation (composition) time, given in the tracks timescale units.
		 * This value is ignored for keyframe-only tracks.
		 */
		int64 PTS = 0;

		/**
		 * The sequential number of the sample.
		 * This value is not written to the file. It is used only as a verification
		 * that this is the sample the `FRequestTrackDataDelegate` has requested
		 * and should be set to that value when adding the sample.
		 */
		uint32 SampleNumber = 0;

		/**
		 * Indicates whether or not this sample is a sync sample that is independently
		 * decodable. If this is a keyframe-only track this value is implicitly handled
		 * as `true` and does not need to be set.
		 */
		bool bIsKeyframe = false;

		/**
		 * Set to `true` when this is the final sample to be added to the track.
		 * Set to `false` if more samples follow or if you cannot make that determination
		 * at this point in time. You may provide an empty sample later that has this
		 * flag set and provides no sample data in `Data`.
		 */
		bool bIsFinalSample = false;
	};

	/**
	 * Adds a sample to be written into the specified track as a response to a sample
	 * request made by the `FRequestTrackDataDelegate`.
	 * Data provided by the ArrayView will be copied into the muxer and can be released
	 * on the user side.
	 */
	virtual bool AddTrackSample(int32 InTrack, const FTrackSample& InTrackSample) = 0;

	/**
	 * Stops muxing and closes the output file.
	 * This can be called at any time to abort the process and must be called
	 * when muxing is complete (either with or without error).
	 */
	virtual bool StopAndClose() = 0;

	/**
	 * Returns a human readable error message if muxing failed.
	 */
	virtual FString GetLastError() = 0;
};

#undef UE_API
