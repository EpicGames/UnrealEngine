// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class MP4Utilities : ModuleRules
	{
		public MP4Utilities(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(new string[] {
				"Core",
                "Media"
            });
		}
	}
}
