// Copyright Epic Games, Inc. All Rights Reserved.

#include "MP4DataReader.h"
#include "HAL/FileManager.h"
#include "Serialization/Archive.h"

#define UE_API MP4UTILITIES_API

namespace MP4Utilities
{

	class FMP4FileDataReaderImpl : public FMP4FileDataReader
	{
	public:
		virtual ~FMP4FileDataReaderImpl()
		{
			Close();
		}

		bool Open(const FString& InFilename) override
		{
			Archive = IFileManager::Get().CreateFileReader(*InFilename, 0);
			if (!Archive)
			{
				LastError = FString::Printf(TEXT("Failed to open file \"%s\""), *InFilename);
				return false;
			}
			TotalFileSize = Archive->TotalSize();
			return true;
		}

		int64 ReadData(void* InOutBuffer, int64 InNumBytes, int64 InFromOffset, FCancellationCheckDelegate InCheckCancellationDelegate) override
		{
			check(InNumBytes >= 0);
			if (!Archive)
			{
				LastError = FString::Printf(TEXT("File reader has not been openend"));
				return EResult::ReadError;
			}
			if (InNumBytes <= 0)
			{
				return 0;
			}
			if (CurrentOffset != InFromOffset)
			{
				check(InFromOffset >= 0 && InFromOffset <= TotalFileSize);
				CurrentOffset = InFromOffset >= 0 ? InFromOffset <= TotalFileSize ? InFromOffset : TotalFileSize : 0;
				Archive->Seek(CurrentOffset);
			}
			if (InNumBytes + CurrentOffset > TotalFileSize)
			{
				InNumBytes = TotalFileSize - CurrentOffset;
			}
			if (InOutBuffer)
			{
				Archive->Serialize(InOutBuffer, InNumBytes);
			}
			else
			{
				Archive->Seek(CurrentOffset + InNumBytes);
			}
			CurrentOffset += InNumBytes;
			return InNumBytes;
		}
		int64 GetTotalFileSize() override
		{ return TotalFileSize; }
		int64 GetCurrentFileOffset() override
		{ return CurrentOffset; }
		bool HasReachedEOF() override
		{ return CurrentOffset >= TotalFileSize; }
		FString GetLastError() override
		{ return LastError; }

	private:
		void Close()
		{
			if (Archive)
			{
				Archive->Close();
				delete Archive;
				Archive = nullptr;
			}
		}
		FString LastError;
		FArchive* Archive = nullptr;
		int64 TotalFileSize = -1;
		int64 CurrentOffset = 0;
	};

	TSharedPtr<FMP4FileDataReader, ESPMode::ThreadSafe> FMP4FileDataReader::Create()
	{
		return MakeShared<FMP4FileDataReaderImpl, ESPMode::ThreadSafe>();
	}










	class FMP4BufferDataReaderImpl : public FMP4BufferDataReader
	{
	public:
		virtual ~FMP4BufferDataReaderImpl() = default;
		FMP4BufferDataReaderImpl(TConstArrayView<const uint8> InDataBuffer) : DataBuffer(MoveTemp(InDataBuffer))
		{ TotalFileSize = DataBuffer.Num(); }

		int64 ReadData(void* InOutBuffer, int64 InNumBytes, int64 InFromOffset, FCancellationCheckDelegate InCheckCancellationDelegate) override
		{
			check(InNumBytes >= 0);
			if (InNumBytes <= 0)
			{
				return 0;
			}
			if (CurrentOffset != InFromOffset)
			{
				check(InFromOffset >= 0 && InFromOffset <= TotalFileSize);
				CurrentOffset = InFromOffset >= 0 ? InFromOffset <= TotalFileSize ? InFromOffset : TotalFileSize : 0;
			}
			if (InNumBytes + CurrentOffset > TotalFileSize)
			{
				InNumBytes = TotalFileSize - CurrentOffset;
			}
			if (InOutBuffer)
			{
				FMemory::Memcpy(InOutBuffer, DataBuffer.GetData() + CurrentOffset, InNumBytes);
			}
			CurrentOffset += InNumBytes;
			return InNumBytes;
		}

		int64 GetTotalFileSize() override
		{ return TotalFileSize; }
		int64 GetCurrentFileOffset() override
		{ return CurrentOffset; }
		bool HasReachedEOF() override
		{ return CurrentOffset >= TotalFileSize; }
		FString GetLastError() override
		{ return FString(); }
	private:
		TConstArrayView<const uint8> DataBuffer;
		int64 TotalFileSize = 0;
		int64 CurrentOffset = 0;
	};

	TSharedPtr<FMP4BufferDataReader, ESPMode::ThreadSafe> FMP4BufferDataReader::Create(TConstArrayView<const uint8> InData)
	{
		return MakeShared<FMP4BufferDataReaderImpl, ESPMode::ThreadSafe>(InData);
	}

} // namespace MP4Utilities
