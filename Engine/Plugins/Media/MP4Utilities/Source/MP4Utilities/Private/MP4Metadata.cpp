// Copyright Epic Games, Inc. All Rights Reserved.

#include "MP4Metadata.h"
#include "Misc/Base64.h"

namespace MP4Utilities
{

namespace FMetadataTags
{
static const TCHAR * const Title = TEXT("Title");
static const TCHAR * const Artist = TEXT("Artist");
static const TCHAR * const Genre = TEXT("Genre");
static const TCHAR * const Date = TEXT("Date");						// Content creation date
static const TCHAR * const Description = TEXT("Description");
static const TCHAR * const LongDescription = TEXT("LongDescription");
static const TCHAR * const Album = TEXT("Album");
static const TCHAR * const Encoder = TEXT("Encoder");

static TMap<uint32, const TCHAR* const> WellKnownItems
{
	{ MakeBoxAtom(0xa9u,'n','a','m'), FMetadataTags::Title },
	{ MakeBoxAtom(0xa9u,'A','R','T'), FMetadataTags::Artist },
	{ MakeBoxAtom(0xa9u,'g','e','n'), FMetadataTags::Genre },
	{ MakeBoxAtom(0xa9u,'d','a','y'), FMetadataTags::Date },
	{ MakeBoxAtom('d','e','s','c'), FMetadataTags::Description },
	{ MakeBoxAtom('l','d','e','s'), FMetadataTags::LongDescription },
	{ MakeBoxAtom(0xa9u,'a','l','b'), FMetadataTags::Album },
	{ MakeBoxAtom(0xa9u,'t','o','o'), FMetadataTags::Encoder }
};
}


class FMP4MetadataParser::FItem : public IMediaMetadataItem
{
public:
	~FItem() = default;
	const FString& GetLanguageCode() const override
	{ return Language; }
	const FString& GetMimeType() const override
	{ return MimeType; }
	const FVariant& GetValue() const override
	{ return Value; }

	FString Language;				// ISO 639-2; if not set the default entry for all languages
	FString MimeType;
	FVariant Value;
	int32 Type = 0;					// Well-known data type (see Quicktime reference)
	FString ToJSONValue() const;
	static TArray<TCHAR> CharsToEscapeInJSON;
	bool operator != (const FItem& Other) const
	{
		return Type != Other.Type || Language != Other.Language || Value != Other.Value;
	}
	bool operator == (const FItem& Other) const
	{
		return !(*this != Other);
	}
};


FMP4MetadataParser::FMP4MetadataParser()
	: Items(MakeShared<TMap<FString, TArray<TUniquePtr<IMediaMetadataItem>>>, ESPMode::ThreadSafe>())
	, NumTotalItems(0)
{
}

void FMP4MetadataParser::AddItem(const FString& InType, const FString& InValue)
{
	// Add a string item.
	FItem* Item = new FItem;
	Item->Type = 1;
	Item->Value = FVariant(InValue);
	++NumTotalItems;
	Items->FindOrAdd(InType).Emplace(TUniquePtr<IMediaMetadataItem>(Item));
}

void FMP4MetadataParser::AddItem(const FString& InType, const FString& InMimeType, const TArray<uint8>& InValue)
{
	if (InMimeType.Equals(TEXT("image/jpeg")))
	{
		FItem* Item = new FItem;
		Item->MimeType = InMimeType;
		Item->Type = 13;
		Item->Value = FVariant(InValue);
		++NumTotalItems;
		Items->FindOrAdd(InType).Emplace(TUniquePtr<IMediaMetadataItem>(Item));
	}
	else if (InMimeType.Equals(TEXT("image/png")))
	{
		FItem* Item = new FItem;
		Item->MimeType = InMimeType;
		Item->Type = 14;
		Item->Value = FVariant(InValue);
		++NumTotalItems;
		Items->FindOrAdd(InType).Emplace(TUniquePtr<IMediaMetadataItem>(Item));
	}
}


FMP4MetadataParser::EResult FMP4MetadataParser::Parse(uint32 InHandler, uint32 InHandlerReserved0, const TArray<FMP4BoxInfo>& InBoxes)
{
	// We only support the Apple iTunes metadata at the moment.
	if (InHandler != MakeBoxAtom('m','d','i','r'))
	{
		return FMP4MetadataParser::EResult::NotSupported;
	}
	/*
		As per ISO/IEC 14496-12:2015 section 8.11.1.2 the 'meta' box must contain a handler ('hdlr'), followed by nothing but optional boxes.
		Of these optional boxes the ones recognized and defined by the document are:
		'pitm' - 8.11.4 Primary Item Box
		'dinf' - 8.7.1 Data Information Box
		'iloc' - 8.11.3 The Item Location Box
		'ipro' - 8.11.5 Item Protection Box
		'iinf' - 8.11.6 Item Information Box
		'iref' - 8.11.12 Item Reference Box
		'idat' - 8.11.11 Item Data Box

		followed by any number of other boxes.

		Presently the boxes mentioned above are not processed since they are apparently not used by any of the tools
		that allow insertion of metadata. It appears that the way iTunes handles metadata has become the de facto standard
		in which there is
		    'meta'
			   'hdlr'
			   'ilst'
			     ....		// See: https://developer.apple.com/library/archive/documentation/QuickTime/QTFF/Metadata/Metadata.html#//apple_ref/doc/uid/TP40000939-CH1-SW26
				   'data'	// See: https://developer.apple.com/library/archive/documentation/QuickTime/QTFF/Metadata/Metadata.html#//apple_ref/doc/uid/TP40000939-CH1-SW27
	*/

	// By some definition the items are contained inside an 'ilst' box, so we look for that one.
	bool bHaveILST = false;
	for(int32 i=0; i<InBoxes.Num(); ++i)
	{
		if (InBoxes[i].Type == MakeBoxAtom('i','l','s','t'))
		{
			// We will actually parse all 'ilst' boxes if for some reason the metadata has
			// been scattered across several.
			bHaveILST = true;
			Parse(InBoxes[i]);
		}
	}
	return bHaveILST ? FMP4MetadataParser::EResult::Success : FMP4MetadataParser::EResult::MissingBox;
}

void FMP4MetadataParser::Parse(const FMP4BoxInfo& InBox)
{
	FMP4AtomReader dr(InBox.Data);
	while(dr.GetNumBytesRemaining() > 8)
	{
		int32 BoxSize;
		uint32 BoxType;
		dr.Read(BoxSize);
		dr.Read(BoxType);
		if (FMetadataTags::WellKnownItems.Contains(BoxType))
		{
			ParseBoxDataList(FMetadataTags::WellKnownItems[BoxType], MakeConstArrayView(dr.GetCurrentDataPointer(), BoxSize-8));
		}
		else
		{
			switch(BoxType)
			{
				case MakeBoxAtom('-','-','-','-'):
				{
					ParseBoxDataiTunes(MakeConstArrayView(dr.GetCurrentDataPointer(), BoxSize-8));
					break;
				}
				default:
				{
					ParseBoxDataList(FString(TEXT("qt."))+Printable4CC(BoxType), MakeConstArrayView(dr.GetCurrentDataPointer(), BoxSize-8));
					break;
				}
			}
		}
		dr.ReadBytes(nullptr, BoxSize - 8);
	}
}


void FMP4MetadataParser::ParseBoxDataList(const FString& AsCategory, TConstArrayView<uint8> InBoxData)
{
	if (InBoxData.Num() <= 8)
	{
		return;
	}

	FMP4AtomReader dr(InBoxData);

#define RETURN_IF_READERROR(expr)	\
	if (!(expr))					\
	{								\
		return;						\
	}								\

	for(int32 CurrentOffset=dr.GetCurrentOffset(), BoxSize=0; dr.GetNumBytesRemaining() > 8; dr.SetCurrentOffset(CurrentOffset), dr.ReadBytes(nullptr, BoxSize - 8))
	{
		uint32 BoxType;
		RETURN_IF_READERROR(dr.Read(BoxSize));
		RETURN_IF_READERROR(dr.Read(BoxType));
		CurrentOffset = dr.GetCurrentOffset();
		if (BoxType != MakeBoxAtom('d','a','t','a'))
		{
			continue;
		}
		uint32 TypeIndicatorAndWellKnownType = 0;
		RETURN_IF_READERROR(dr.Read(TypeIndicatorAndWellKnownType));
		if ((TypeIndicatorAndWellKnownType >> 24) != 0)
		{
			continue;
		}
		uint16 CountryIndicator=0;
		RETURN_IF_READERROR(dr.Read(CountryIndicator));
		uint16 LanguageIndicator=0;
		RETURN_IF_READERROR(dr.Read(LanguageIndicator));
		int32 NumRemainingDataBytes = BoxSize - 8 - (dr.GetCurrentOffset() - CurrentOffset);

		TUniquePtr<FItem> Item(new FItem);

		// We do not handle the country indicator at the moment.
		// Likewise, the language indicator must be a directly specified language code since we do not handle
		// language tables.
		if (LanguageIndicator >= 1 && LanguageIndicator <= 255)
		{
			// This would be an index into the language table (which is a 'lang' box somewhere under the 'meta' box).
			continue;
		}
		if (LanguageIndicator)
		{
			char Lang[3];
			Lang[0] = (char)(0x60 + ((LanguageIndicator & 0x7c00) >> 10));
			Lang[1] = (char)(0x60 + ((LanguageIndicator & 0x03e0) >> 5));
			Lang[2] = (char)(0x60 + (LanguageIndicator & 0x001f));
			Item->Language = FString::ConstructFromPtrSize(Lang, 3);
		}

		bool bSet = true;
		int32 WKT = (int32)(TypeIndicatorAndWellKnownType & 0x00ffffffU);
		switch(WKT)
		{
			// UTF-8 string
			case 1:
			{
				FString String;
				if ((bSet = dr.ReadStringUTF8(String, NumRemainingDataBytes)) == true)
				{
					Item->Value = String;
					Item->Type = 1;
				}
				break;
			}
/*
			// UTF-16 string
			case 2:
			{
				FString String;
				if ((bSet = dr.ReadStringUTF16(String, NumRemainingDataBytes)) == true)
				{
					Item->Value = String;
					Item->Type = 1;
				}
				break;
			}
*/
			case 13:	// JPEG image
			case 14:	// PNG image
			case 27:	// BMP image
			{
				TArray<uint8> Image(dr.GetCurrentDataPointer(), NumRemainingDataBytes);
				Item->Value = MoveTemp(Image);
				Item->Type = WKT;
				if (WKT == 13)
				{
					Item->MimeType = TEXT("image/jpeg");
				}
				else if (WKT == 14)
				{
					Item->MimeType = TEXT("image/png");
				}
				else if (WKT == 27)
				{
					Item->MimeType = TEXT("image/bmp");
				}
				break;
			}

			// Float
			case 23:
			{
				float Flt;
				if ((bSet = dr.ReadAsNumber(Flt)) == true)
				{
					// Set as double!
					Item->Value = (double)Flt;
					Item->Type = 24;
				}
				break;
			}
			// Double
			case 24:
			{
				double Dbl;
				if ((bSet = dr.ReadAsNumber(Dbl)) == true)
				{
					Item->Value = Dbl;
					Item->Type = 24;
				}
				break;
			}
			// Signed integers
			case 65:	// int8
			case 66:	// int16
			case 67:	// int32
			case 74:	// int64
			case 21:	// 1-4 byte signed integer
			{
				int64 Number;
				if ((bSet = dr.ReadAsNumber(Number, WKT==65?1:WKT==66?2:WKT==67?4:WKT==74?8:NumRemainingDataBytes)) == true)
				{
					Item->Value = Number;
					Item->Type = 74;
				}
				break;
			}
			// Unsigned integers
			case 75:	// uint8
			case 76:	// uint16
			case 77:	// uint32
			case 78:	// uint64
			case 22:	// 1-4 byte unsigned integer
			{
				uint64 Number;
				if ((bSet = dr.ReadAsNumber(Number, WKT==75?1:WKT==76?2:WKT==77?4:WKT==78?8:NumRemainingDataBytes)) == true)
				{
					Item->Value = Number;
					Item->Type = 78;
				}
				break;
			}

			// Not handled here.
			default:
			{
				bSet = false;
				break;
			}
		}

		if (bSet)
		{
			TUniquePtr<IMediaMetadataItem> mi(Item.Release());
			Items->FindOrAdd(AsCategory).Emplace(MoveTemp(mi));
			++NumTotalItems;
		}
	}
#undef RETURN_IF_READERROR
}

void FMP4MetadataParser::ParseBoxDataiTunes(TConstArrayView<uint8> InBoxData)
{
	if (InBoxData.Num() <= 8)
	{
		return;
	}

	FMP4AtomReader dr(InBoxData);

#define RETURN_IF_READERROR(expr)	\
	if (!(expr))					\
	{								\
		return;						\
	}								\

	FString Name;
	for(int32 CurrentOffset=dr.GetCurrentOffset(), BoxSize=0; dr.GetNumBytesRemaining() > 8; dr.SetCurrentOffset(CurrentOffset), dr.ReadBytes(nullptr, BoxSize - 8))
	{
		uint32 BoxType;
		RETURN_IF_READERROR(dr.Read(BoxSize));
		RETURN_IF_READERROR(dr.Read(BoxType));
		CurrentOffset = dr.GetCurrentOffset();
		if (BoxType == MakeBoxAtom('m','e','a','n'))
		{
			// 4 reserved bytes follow
			RETURN_IF_READERROR(dr.ReadBytes(nullptr, 4));
			FString Meaning;
			int32 NumRemainingDataBytes = BoxSize - 8 - (dr.GetCurrentOffset() - CurrentOffset);
			RETURN_IF_READERROR(dr.ReadStringUTF8(Meaning, NumRemainingDataBytes));
			if (!Meaning.Equals(TEXT("com.apple.iTunes")))
			{
				return;
			}
		}
		else if (BoxType == MakeBoxAtom('n','a','m','e'))
		{
			// 4 reserved bytes follow
			RETURN_IF_READERROR(dr.ReadBytes(nullptr, 4));
			int32 NumRemainingDataBytes = BoxSize - 8 - (dr.GetCurrentOffset() - CurrentOffset);
			RETURN_IF_READERROR(dr.ReadStringUTF8(Name, NumRemainingDataBytes));
		}
		else if (BoxType == MakeBoxAtom('d','a','t','a'))
		{
			ParseBoxDataList(FString(TEXT("iTunes."))+Name, MakeConstArrayView(dr.GetCurrentDataPointer()-8, BoxSize));
		}
	}
#undef RETURN_IF_READERROR
}

bool FMP4MetadataParser::IsDifferentFrom(const FMP4MetadataParser& Other)
{
	// Different number of items or map entries means the metadata can't be identical.
	if (NumTotalItems != Other.NumTotalItems || Items->Num() != Other.Items->Num())
	{
		return true;
	}

	// Check if the maps keys or items are different.
	for(auto& ItemMapIterator : *Items)
	{
		const TArray<TUniquePtr<IMediaMetadataItem>>* OtherKeyItems = Other.Items->Find(ItemMapIterator.Key);
		if (!OtherKeyItems || ItemMapIterator.Value.Num() != OtherKeyItems->Num())
		{
			return true;
		}
		// Expensive item by item test
		for(auto& ItemIterator : ItemMapIterator.Value)
		{
			if (!OtherKeyItems->ContainsByPredicate([&ItemIterator](const TUniquePtr<IMediaMetadataItem>& CompItem)
			{
				return *static_cast<FItem*>(ItemIterator.Get()) == *static_cast<FItem*>(CompItem.Get());
			}))
			{
				return true;
			}
		}
	}
	return false;
}


TSharedPtr<TMap<FString, TArray<TUniquePtr<IMediaMetadataItem>>>, ESPMode::ThreadSafe> FMP4MetadataParser::GetMediaStreamMetadata() const
{
	return Items;
}


FString FMP4MetadataParser::GetAsJSON() const
{
	FString JSON(TEXT("{"));
	bool bFirstItem = true;
	for(auto& Item : *Items)
	{
		bool bFirstItemValue = true;
		const TArray<TUniquePtr<IMediaMetadataItem>>& List=Item.Value;
		FString ItemJSON;
		for(int32 i=0; i<List.Num(); ++i)
		{
			const FItem* it = static_cast<const FItem*>(List[i].Get());
			FString ItemValue = it->ToJSONValue();
			if (ItemValue.Len())
			{
				if (!bFirstItemValue)
				{
					ItemJSON.AppendChar(TCHAR(','));
				}
				bFirstItemValue = false;
				ItemJSON.Append(FString::Printf(TEXT("\"%s\":"), *(it->Language)));
				ItemJSON.Append(MoveTemp(ItemValue));
			}
		}
		if (ItemJSON.Len())
		{
			if (!bFirstItem)
			{
				JSON.AppendChar(TCHAR(','));
			}
			bFirstItem = false;

			JSON.Append(FString::Printf(TEXT("\"%s\":{"), *Item.Key));
			JSON.Append(MoveTemp(ItemJSON));
			JSON.Append(TEXT("}"));
		}
	}
	JSON.Append(TEXT("}"));
	return JSON;
}

FString FMP4MetadataParser::FItem::ToJSONValue() const
{
	switch(Type)
	{
		// String
		case 1:
		{
			return FString::Printf(TEXT("\"%s\""), *Value.GetValue<FString>().ReplaceCharWithEscapedChar(&CharsToEscapeInJSON));
		}
		// Double
		case 24:
		{
			return FString::Printf(TEXT("%f"), Value.GetValue<double>());
		}
		// Signed integer
		case 74:
		{
			int64 v = Value.GetValue<int64>();
			if (v >= -(1LL<<53) && v <= (1LL<<53)-1)
			{
				return FString::Printf(TEXT("%lld"), (long long int)v);
			}
		}
		// Unsigned integer
		case 78:
		{
			uint64 v = Value.GetValue<uint64>();
			if (v <= (1LL<<53)-1)
			{
				return FString::Printf(TEXT("%lld"), (long long int)v);
			}
		}
		// JPEG image
		case 13:
		{
			FString js(TEXT("{\"mimetype\":\"image/jpeg\",\"data\":\""));
			js.Append(FBase64::Encode(Value.GetValue<TArray<uint8>>()));
			js.Append(TEXT("\"}"));
			return js;
		}
		// PNG image
		case 14:
		{
			FString js(TEXT("{\"mimetype\":\"image/png\",\"data\":\""));
			js.Append(FBase64::Encode(Value.GetValue<TArray<uint8>>()));
			js.Append(TEXT("\"}"));
			return js;
		}
		default:
		{
			break;
		}
	}
	return FString();
}
TArray<TCHAR> FMP4MetadataParser::FItem::CharsToEscapeInJSON = { TCHAR('\n'), TCHAR('\r'), TCHAR('\t'), TCHAR('\\'), TCHAR('\"') };



} // namespace MP4Utilities
