// Copyright Epic Games, Inc. All Rights Reserved.

#include "MP4Utilities.h"
#include "MP4UtilitiesModule.h"

namespace MP4Utilities
{
	TSharedPtr<FMP4BoxData, ESPMode::ThreadSafe> CreateBoxDataFromBuffer(TConstArrayView<uint8> InBoxDataBuffer, int64 InBoxDataFileOffset)
	{
		TSharedPtr<FMP4BoxData, ESPMode::ThreadSafe> bd = MakeShared<FMP4BoxData, ESPMode::ThreadSafe>();
		bd->DataBuffer = InBoxDataBuffer;
		FMP4AtomReader ar(bd->DataBuffer);
		return ar.ParseIntoBoxInfo(*bd, InBoxDataFileOffset) ? bd : nullptr;
	}


	FMP4AtomReader::FMP4AtomReader(const TConstArrayView<uint8>& InData)
		: DataPtr(InData.GetData()), DataSize(InData.Num()), CurrentOffset(0)
	{
	}

	int64 FMP4AtomReader::GetCurrentOffset() const
	{
		return CurrentOffset;
	}

	int64 FMP4AtomReader::GetNumBytesRemaining() const
	{
		return DataSize - GetCurrentOffset();
	}

	const uint8* FMP4AtomReader::GetCurrentDataPointer() const
	{
		return GetNumBytesRemaining() ? DataPtr + GetCurrentOffset() : nullptr;
	}

	void FMP4AtomReader::SetCurrentOffset(int64 InNewOffset)
	{
		check(InNewOffset >= 0 && InNewOffset <= DataSize);
		if (InNewOffset >= 0 && InNewOffset <= DataSize)
		{
			CurrentOffset = InNewOffset;
		}
	}

	bool FMP4AtomReader::ReadVersionAndFlags(uint8& OutVersion, uint32& OutFlags)
	{
		uint32 VersionAndFlags = 0;
		if (!Read(VersionAndFlags))
		{
			return false;
		}
		OutVersion = (uint8)(VersionAndFlags >> 24);
		OutFlags = VersionAndFlags & 0x00ffffff;
		return true;
	}

	bool FMP4AtomReader::ReadString(FString& OutString, uint16 InNumBytes)
	{
		OutString.Empty();
		if (InNumBytes == 0)
		{
			return true;
		}
		TArray<uint8> Buf;
		Buf.AddUninitialized(InNumBytes);
		if (ReadBytes(Buf.GetData(), InNumBytes))
		{
			// Check for UTF16 BOM
			if (InNumBytes >= 2 && ((Buf[0] == 0xff && Buf[1] == 0xfe) || (Buf[0] == 0xfe && Buf[1] == 0xff)))
			{
				// String uses UTF16, which is not supported
				return false;
			}
			FUTF8ToTCHAR cnv((const ANSICHAR*)Buf.GetData(), InNumBytes);
			OutString = FString::ConstructFromPtrSize(cnv.Get(), cnv.Length());
			return true;
		}
		return false;
	}

	bool FMP4AtomReader::ReadStringUTF8(FString& OutString, int32 InNumBytes)
	{
		OutString.Empty();
		if (InNumBytes == 0)
		{
			return true;
		}
		else if (InNumBytes < 0)
		{
			InNumBytes = GetNumBytesRemaining();
			check(InNumBytes >= 0);
			if (InNumBytes < 0)
			{
				return false;
			}
		}
		TArray<uint8> Buf;
		Buf.AddUninitialized(InNumBytes);
		if (ReadBytes(Buf.GetData(), InNumBytes))
		{
			FUTF8ToTCHAR cnv((const ANSICHAR*)Buf.GetData(), InNumBytes);
			OutString = FString::ConstructFromPtrSize(cnv.Get(), cnv.Length());
			return true;
		}
		return false;
	}

	bool FMP4AtomReader::ReadStringUTF16(FString& OutString, int32 InNumBytes)
	{
		OutString.Empty();
		if (InNumBytes == 0)
		{
			return true;
		}
		else if (InNumBytes < 0)
		{
			InNumBytes = GetNumBytesRemaining();
			check(InNumBytes >= 0);
			if (InNumBytes < 0)
			{
				return false;
			}
		}
		TArray<uint8> Buf;
		Buf.AddUninitialized(InNumBytes);
		if (ReadBytes(Buf.GetData(), InNumBytes))
		{
		// TODO
			unimplemented();
	/*
			FUTF8ToTCHAR cnv((const ANSICHAR*)Buf.GetData(), InNumBytes);
			OutString = FString(cnv.Length(), cnv.Get());
			return true;
	*/
		}
		return false;
	}

	bool FMP4AtomReader::ReadNullTerminatedStringUTF8(FString& OutString)
	{
		TArray<uint8> TempBuf;
		TempBuf.Reserve(256);
		while(GetNumBytesRemaining())
		{
			uint8 b;
			if (!Read(b))
			{
				return false;
			}
			if (b == 0)
			{
				break;
			}
			TempBuf.Add(b);
		}
		if (TempBuf.Num())
		{
			// Check that this is not accidentally a Pascal string with the length (as a byte) as the first value.
			if (TempBuf.Num() < 256 && TempBuf[0] == TempBuf.Num())
			{
				TempBuf.RemoveAt(0);
			}
			FUTF8ToTCHAR cnv((const ANSICHAR*)TempBuf.GetData(), TempBuf.Num());
			OutString = FString::ConstructFromPtrSize(cnv.Get(), cnv.Length());
			return true;
		}
		return false;
	}


	bool FMP4AtomReader::ReadAsNumber(uint64& OutValue, int32 InNumBytes)
	{
		OutValue = 0;
		if (InNumBytes < 0 || InNumBytes > 8)
		{
			return false;
		}
		for(int32 i=0; i<InNumBytes; ++i)
		{
			uint8 d;
			if (!Read(d))
			{
				return false;
			}
			OutValue = (OutValue << 8) | d;
		}
		return true;
	}
	bool FMP4AtomReader::ReadAsNumber(int64& OutValue, int32 InNumBytes)
	{
		OutValue = 0;
		if (InNumBytes < 0 || InNumBytes > 8)
		{
			return false;
		}
		for(int32 i=0; i<InNumBytes; ++i)
		{
			uint8 d;
			if (!Read(d))
			{
				return false;
			}
			if (i==0 && d>127)
			{
				OutValue = -1;
			}
			OutValue = (OutValue << 8) | d;
		}
		return true;
	}
	bool FMP4AtomReader::ReadAsNumber(float& OutValue)
	{
		uint32 Flt;
		if (Read(Flt))
		{
			OutValue = *reinterpret_cast<float*>(&Flt);
			return true;
		}
		return false;
	}
	bool FMP4AtomReader::ReadAsNumber(double& OutValue)
	{
		uint64 Dbl;
		if (Read(Dbl))
		{
			OutValue = *reinterpret_cast<double*>(&Dbl);
			return true;
		}
		return false;
	}

	bool FMP4AtomReader::ReadBytes(void* Buffer, int32 InNumBytes)
	{
		return ReadData(Buffer, InNumBytes) == InNumBytes;
	}

	int32 FMP4AtomReader::ReadData(void* IntoBuffer, int32 NumBytesToRead)
	{
		if (NumBytesToRead <= 0)
		{
			return 0;
		}
		int64 NumAvail = DataSize - CurrentOffset;
		if (NumAvail >= NumBytesToRead)
		{
			if (IntoBuffer)
			{
				FMemory::Memcpy(IntoBuffer, DataPtr + CurrentOffset, NumBytesToRead);
			}
			CurrentOffset += NumBytesToRead;
			return NumBytesToRead;
		}
		return -1;
	}

	bool FMP4AtomReader::ParseIntoBoxInfo(FMP4BoxInfo& OutBoxInfo, int64 InAtFileOffset)
	{
		// Clear output with default values before continuing.
		OutBoxInfo = FMP4BoxInfo();
		uint32 BoxSize, BoxType;
		if (!Read(BoxSize) || !Read(BoxType))
		{
			return false;
		}
		OutBoxInfo.Offset = InAtFileOffset;
		OutBoxInfo.Size = (int64) BoxSize;
		OutBoxInfo.Type = BoxType;
	#if !UE_BUILD_SHIPPING
		OutBoxInfo.Name[0] = (char) ((BoxType >> 24) & 255);
		OutBoxInfo.Name[1] = (char) ((BoxType >> 16) & 255);
		OutBoxInfo.Name[2] = (char) ((BoxType >>  8) & 255);
		OutBoxInfo.Name[3] = (char) ((BoxType >>  0) & 255);
		OutBoxInfo.Name[4] = 0;
	#endif
		OutBoxInfo.DataOffset = 8;
		// Check the box size value.
		if (OutBoxInfo.Size == 1)
		{
			uint64 BoxSize64;
			if (!Read(BoxSize64))
			{
				return false;
			}
			OutBoxInfo.DataOffset += 8;
			OutBoxInfo.Size = (int64) BoxSize64;
		}
		// Is the box type a UUID ?
		if (OutBoxInfo.Type == MakeBoxAtom('u','u','i','d'))
		{
			// Read additional 16 bytes for the UUID
			if (ReadData(OutBoxInfo.UUID, 16) != 16)
			{
				return false;
			}
			OutBoxInfo.DataOffset += 16;
		}
		OutBoxInfo.Data = MakeConstArrayView(GetCurrentDataPointer(), OutBoxInfo.Size ? OutBoxInfo.Size - OutBoxInfo.DataOffset : GetNumBytesRemaining());
		return true;
	}





	bool FMP4BoxLocator::LocateAndReadRootBoxesInternal(TArray<TSharedPtr<FMP4BoxData, ESPMode::ThreadSafe>>& OutBoxInfos, const TSharedPtr<IMP4DataReaderBase, ESPMode::ThreadSafe>& InDataReader, const TArray<uint32>& InFirstBoxes, const TArray<uint32>& InStopAfterBoxes, bool bStopWithMDAT, bool bCheckWellKnown, const TArray<uint32>& InReadDataOfBoxes, IMP4DataReaderBase::FCancellationCheckDelegate InCheckCancellationDelegate)
	{
		static const TArray<uint32> kWellKnownRootBoxes {
			MakeBoxAtom('f','t','y','p'), MakeBoxAtom('o','t','y','p'), MakeBoxAtom('p','d','i','n'),
			MakeBoxAtom('m','o','o','v'), MakeBoxAtom('m','o','o','f'), MakeBoxAtom('m','f','r','a'),
			MakeBoxAtom('m','d','a','t'), MakeBoxAtom('f','r','e','e'), MakeBoxAtom('s','k','i','p'),
			MakeBoxAtom('i','m','d','a'), MakeBoxAtom('m','e','t','a'), MakeBoxAtom('s','t','y','p'),
			MakeBoxAtom('s','i','d','x'), MakeBoxAtom('s','s','i','x'), MakeBoxAtom('p','r','f','t'),
			MakeBoxAtom('u','u','i','d')
		};

		// We NEVER want to read the `mdat` box here!
		if (!InDataReader.IsValid() || InReadDataOfBoxes.Contains(MakeBoxAtom('m','d','a','t')))
		{
			return false;
		}
#if !UE_BUILD_SHIPPING
		if (bCheckWellKnown)
		{
			auto WarnNotWellKnown = [&](const TArray<uint32>& InBoxes) -> void
			{
				for(int32 i=0; i<InBoxes.Num(); ++i)
				{
					if (!kWellKnownRootBoxes.Contains(InBoxes[i]))
					{
						UE_LOGF(LogMP4Utilities, Warning, "Box `%ls` is not a well-known root box", *GetPrintableBoxAtom(InBoxes[i]));
					}
				}
			};
			WarnNotWellKnown(InFirstBoxes);
			WarnNotWellKnown(InStopAfterBoxes);
			WarnNotWellKnown(InReadDataOfBoxes);
		}
#endif

		CurrentOffset = InDataReader->GetCurrentFileOffset();

		#define CHECK_READ(NumReq) \
			if (NumRead == IMP4DataReaderBase::EResult::Canceled) \
			{ return false;	} \
			else if (NumRead == IMP4DataReaderBase::EResult::ReadError) \
			{ LastError = InDataReader->GetLastError(); return false; } \
			else if (/*NumRead == IMP4DataReaderBase::EResult::ReachedEOF ||*/ NumRead != NumReq) \
			{ LastError = FString::Printf(TEXT("File truncated. Cannot read %lld bytes from offset %lld"), (long long int)NumReq, (long long int)CurrentOffset+BoxInternalOffset); return false; }

		int64 TotalFileSize = -1;
		for(int32 BoxNum=0; ;++BoxNum)
		{
			union UBuf
			{
				uint64 As64[2];
				uint32 As32[4];
				uint8 As8[16];
			};
			UBuf BoxSizeAndType;

			// Read 8 bytes
			uint32 BoxInternalOffset = 0;
			int64 NumRead = InDataReader->ReadData(BoxSizeAndType.As64, 8, CurrentOffset, InCheckCancellationDelegate);
			CHECK_READ(8);

			BoxInternalOffset = 8;
			TSharedPtr<FMP4BoxData, ESPMode::ThreadSafe> bi = MakeShared<FMP4BoxData, ESPMode::ThreadSafe>();
			bi->Size = (int64) GetFromBigEndian(BoxSizeAndType.As32[0]);
			bi->Offset = CurrentOffset;
			bi->Type = GetFromBigEndian(BoxSizeAndType.As32[1]);
	#if !UE_BUILD_SHIPPING
			bi->Name[0] = BoxSizeAndType.As8[4]; bi->Name[1] = BoxSizeAndType.As8[5]; bi->Name[2] = BoxSizeAndType.As8[6]; bi->Name[3] = BoxSizeAndType.As8[7]; bi->Name[4] = 0;
	#endif
			//UE_LOGF(LogMP4Utilities, Warning, "[%p]: box `%ls` (%8x); %lld @ %lld", this, *GetPrintableBoxAtom(bi->Type), bi->Type, (long long int) bi->Size, (long long int) bi->Offset);

			// After having read the first few bytes we should now know the overall filesize.
			if (BoxNum == 0)
			{
				TotalFileSize = InDataReader->GetTotalFileSize();
				if (InFirstBoxes.Num() && !InFirstBoxes.Contains(bi->Type))
				{
					LastError = TEXT("Invalid mp4 file: First box is not of expected type");
					return false;
				}
			}

			// Check the box size value.
			if (bi->Size == 0)
			{
				// Zero size means "until the end of the file".
				bi->Size = TotalFileSize > 0 ? TotalFileSize - CurrentOffset : -1;
			}
			else if (bi->Size == 1)
			{
				// A size of 1 indicates that the size is expressed as a 64 bit value following the box type.
				// Read additional 8 bytes for the 64 bit length
				NumRead = InDataReader->ReadData(BoxSizeAndType.As64, 8, CurrentOffset+BoxInternalOffset, InCheckCancellationDelegate);
				CHECK_READ(8);
				BoxInternalOffset += 8;
				bi->Size = (int64) GetFromBigEndian(BoxSizeAndType.As64[0]);
			}

			// Is the box type a UUID ?
			if (bi->Type == MakeBoxAtom('u','u','i','d'))
			{
				// Read additional 16 bytes for the UUID
				NumRead = InDataReader->ReadData(bi->UUID, 16, CurrentOffset+BoxInternalOffset, InCheckCancellationDelegate);
				CHECK_READ(16);
				BoxInternalOffset += 16;
			}
			// Shall we read this box?
			if (bi->Type != MakeBoxAtom('m','d','a','t'))
			{
				if (!bCheckWellKnown || kWellKnownRootBoxes.Contains(bi->Type))
				{
					if (InReadDataOfBoxes.IsEmpty() || InReadDataOfBoxes.Contains(bi->Type))
					{
						// Size safety check.
						const uint64 kMaxSafeBoxSize = 1024 * 1024 * 128;
						if ((uint64)bi->Size <= kMaxSafeBoxSize)
						{
							bi->DataBuffer.SetNumUninitialized(bi->Size-BoxInternalOffset);
							bi->Data = MakeConstArrayView(bi->DataBuffer);
							NumRead = InDataReader->ReadData(bi->DataBuffer.GetData(), bi->Size-BoxInternalOffset, CurrentOffset+BoxInternalOffset, InCheckCancellationDelegate);
						}
						else
						{
							UE_LOGF(LogMP4Utilities, Warning, "Skipping over box `%ls` because it is suspiciously large (%lld bytes)", *GetPrintableBoxAtom(bi->Type), (long long int) bi->Size);
							NumRead = InDataReader->ReadData(nullptr, bi->Size-BoxInternalOffset, CurrentOffset+BoxInternalOffset, InCheckCancellationDelegate);
						}
					}
					else
					{
						NumRead = InDataReader->ReadData(nullptr, bi->Size-BoxInternalOffset, CurrentOffset+BoxInternalOffset, InCheckCancellationDelegate);
					}
				}
				else
				{
					UE_LOGF(LogMP4Utilities, Verbose, "Skipping over box `%ls` because it is not well known", *GetPrintableBoxAtom(bi->Type));
					NumRead = InDataReader->ReadData(nullptr, bi->Size-BoxInternalOffset, CurrentOffset+BoxInternalOffset, InCheckCancellationDelegate);
				}
			}
			else
			{
				if (!bStopWithMDAT)
				{
					NumRead = InDataReader->ReadData(nullptr, bi->Size-BoxInternalOffset, CurrentOffset+BoxInternalOffset, InCheckCancellationDelegate);
				}
				else
				{
					bi->DataOffset = BoxInternalOffset;
					OutBoxInfos.Emplace(MoveTemp(bi));
					return true;
				}
			}
			CHECK_READ(bi->Size - BoxInternalOffset);
			bi->DataOffset = BoxInternalOffset;

			// Advance the current offset, whether we have read the box or not.
			CurrentOffset += bi->Size;
			check(CurrentOffset == InDataReader->GetCurrentFileOffset());
			bool bStopNow = InStopAfterBoxes.Contains(bi->Type);
			OutBoxInfos.Emplace(MoveTemp(bi));
			if (bStopNow || InDataReader->HasReachedEOF())
			{
				return true;
			}
		}
		#undef CHECK_READ
	}


}// namespace MP4Utilities
