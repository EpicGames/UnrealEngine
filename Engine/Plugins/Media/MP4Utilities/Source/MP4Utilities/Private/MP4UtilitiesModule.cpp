// Copyright Epic Games, Inc. All Rights Reserved.

#include "MP4UtilitiesModule.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "MP4UtilitiesModule"

DEFINE_LOG_CATEGORY(LogMP4Utilities);

class FMP4UtilitiesModule : public IModuleInterface
{
public:
	void StartupModule() override
	{
	}

	void ShutdownModule() override
	{
	}

	bool SupportsDynamicReloading() override
	{
		return false;
	}

private:
};

IMPLEMENT_MODULE(FMP4UtilitiesModule, MP4Utilities);

#undef LOCTEXT_NAMESPACE
