// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

#include "IMediaMetadataItem.h"

#include "MP4Utilities.h"

#define UE_API MP4UTILITIES_API

namespace MP4Utilities
{
	/**
	 * This class parses metadata embedded in an ISO/IEC 14496-12 file.
	 * Presently only the structure as used and defined by Apple iTunes is supported.
	 */
	class FMP4MetadataParser
	{
	public:
		UE_API FMP4MetadataParser();
		enum class EResult
		{
			Success,
			NotSupported,
			MissingBox
		};
		UE_API EResult Parse(uint32 InHandler, uint32 InHandlerReserved0, const TArray<FMP4BoxInfo>& InBoxes);
		UE_API bool IsDifferentFrom(const FMP4MetadataParser& Other);
		UE_API FString GetAsJSON() const;
		UE_API TSharedPtr<TMap<FString, TArray<TUniquePtr<IMediaMetadataItem>>>, ESPMode::ThreadSafe> GetMediaStreamMetadata() const;

		UE_API void AddItem(const FString& InType, const FString& InValue);
		UE_API void AddItem(const FString& InType, const FString& InMimeType, const TArray<uint8>& InValue);
	private:
		class FItem;

		FString PrintableBoxAtom(const uint32 InAtom);
		void Parse(const FMP4BoxInfo& InBox);
		void ParseBoxDataList(const FString& AsCategory, TConstArrayView<uint8> InBoxData);
		void ParseBoxDataiTunes(TConstArrayView<uint8> InBoxData);

		TSharedPtr<TMap<FString, TArray<TUniquePtr<IMediaMetadataItem>>>, ESPMode::ThreadSafe> Items;
		uint32 NumTotalItems = 0;
	};

} // namespace MP4Utilities

#undef UE_API
