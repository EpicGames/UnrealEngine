// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Templates/SharedPointer.h"
#include "Math/NumericLimits.h"
#include "Math/BigInt.h"
#include "Misc/Timespan.h"

#include "MP4DataReader.h"

#define UE_API MP4UTILITIES_API

namespace MP4Utilities
{
#if !PLATFORM_LITTLE_ENDIAN
	static constexpr uint8 GetFromBigEndian(uint8 value)		{ return value; }
	static constexpr int8 GetFromBigEndian(int8 value)			{ return value; }
	static constexpr uint16 GetFromBigEndian(uint16 value)		{ return value; }
	static constexpr int16 GetFromBigEndian(int16 value)		{ return value; }
	static constexpr int32 GetFromBigEndian(int32 value)		{ return value; }
	static constexpr uint32 GetFromBigEndian(uint32 value)		{ return value; }
	static constexpr int64 GetFromBigEndian(int64 value)		{ return value; }
	static constexpr uint64 GetFromBigEndian(uint64 value)		{ return value; }
#else
	static constexpr uint16 EndianSwap(uint16 value)			{ return (value >> 8) | (value << 8); }
	static constexpr int16 EndianSwap(int16 value)				{ return int16(EndianSwap(uint16(value))); }
	static constexpr uint32 EndianSwap(uint32 value)			{ return (value << 24) | ((value & 0xff00) << 8) | ((value >> 8) & 0xff00) | (value >> 24); }
	static constexpr int32 EndianSwap(int32 value)				{ return int32(EndianSwap(uint32(value))); }
	static constexpr uint64 EndianSwap(uint64 value)			{ return (uint64(EndianSwap(uint32(value & 0xffffffffU))) << 32) | uint64(EndianSwap(uint32(value >> 32))); }
	static constexpr int64 EndianSwap(int64 value)				{ return int64(EndianSwap(uint64(value)));}
	static constexpr uint8 GetFromBigEndian(uint8 value)		{ return value; }
	static constexpr int8 GetFromBigEndian(int8 value)			{ return value; }
	static constexpr uint16 GetFromBigEndian(uint16 value)		{ return EndianSwap(value); }
	static constexpr int16 GetFromBigEndian(int16 value)		{ return EndianSwap(value); }
	static constexpr int32 GetFromBigEndian(int32 value)		{ return EndianSwap(value); }
	static constexpr uint32 GetFromBigEndian(uint32 value)		{ return EndianSwap(value); }
	static constexpr int64 GetFromBigEndian(int64 value)		{ return EndianSwap(value); }
	static constexpr uint64 GetFromBigEndian(uint64 value)		{ return EndianSwap(value); }
#endif


	static constexpr uint32 MakeBoxAtom(const uint8 A, const uint8 B, const uint8 C, const uint8 D)
	{
		return (static_cast<uint32>(A) << 24) | (static_cast<uint32>(B) << 16) | (static_cast<uint32>(C) << 8) | static_cast<uint32>(D);
	}

	static FString GetPrintableBoxAtom(uint32 InAtom)
	{
		TCHAR tc[4];
		tc[0] = (TCHAR) ((InAtom >> 24) & 255);
		tc[1] = (TCHAR) ((InAtom >> 16) & 255);
		tc[2] = (TCHAR) ((InAtom >>  8) & 255);
		tc[3] = (TCHAR) ((InAtom >>  0) & 255);
		for(int32 i=0;i<4; ++i)
		{
			tc[i] = tc[i] >= 32 && tc[i] <= 127 ? tc[i] : TCHAR(' ');
		}
		return FString::ConstructFromPtrSize(tc, 4);
	}

	static FString Printable4CC(const uint32 In4CC)
	{
		FString Out;
		// Not so much just printable as alphanumeric.
		for(uint32 i=0, Atom=In4CC; i<4; ++i, Atom<<=8)
		{
			int32 v = Atom >> 24;
			if ((v >= 'A' && v <= 'Z') || (v >= 'a' && v <= 'z') || (v >= '0' && v <= '9') || v == '_' || v == '.')
			{
				Out.AppendChar(v);
			}
			else
			{
				// Not alphanumeric, return it as a hex string.
				return FString::Printf(TEXT("%08x"), In4CC);
			}
		}
		return Out;
	}

	struct FMP4BoxInfo
	{
		TConstArrayView<uint8> Data;
		uint8 UUID[16] {};
		int64 Size = 0;
		int64 Offset = 0;
		uint32 Type = 0;
		uint32 DataOffset = 0;
	#if !UE_BUILD_SHIPPING
		char Name[5]{0};
	#endif
		TArray<uint8> GetBoxDataRAW() const
		{
			TArray<uint8> Raw;
			Raw.AddUninitialized(8 + Data.Num());
			uint32* bd = reinterpret_cast<uint32*>(Raw.GetData());
			*bd++ = EndianSwap(static_cast<uint32>(Raw.Num()));
			*bd++ = EndianSwap(Type);
			FMemory::Memcpy(bd, Data.GetData(), Data.Num());
			return Raw;
		}
	};

	/**
	 * Box information along with the actual box data to be retained
	 * in memory.
	 */
	struct FMP4BoxData : public FMP4BoxInfo
	{
		TArray<uint8> DataBuffer;
	};


	// Create an FMP4BoxData from a transient data buffer. The data is copied into the FMP4BoxData.
	// If the box data fails to be recognized as an mp4 box a nullptr is returned.
	UE_API TSharedPtr<FMP4BoxData, ESPMode::ThreadSafe> CreateBoxDataFromBuffer(TConstArrayView<uint8> InBoxDataBuffer, int64 InBoxDataFileOffset);


	class FMP4AtomReader
	{
	public:
		UE_API FMP4AtomReader(const TConstArrayView<uint8>& InData);

		UE_API bool ParseIntoBoxInfo(FMP4BoxInfo& OutBoxInfo, int64 InAtFileOffset);

		UE_API int64 GetCurrentOffset() const;
		UE_API int64 GetNumBytesRemaining() const;
		UE_API const uint8* GetCurrentDataPointer() const;
		UE_API void SetCurrentOffset(int64 InNewOffset);

		template <typename T>
		bool Read(T& OutValue)
		{
			T Temp = 0;
			int64 NumRead = ReadData(&Temp, sizeof(T));
			if (NumRead == sizeof(T))
			{
				OutValue = ValueFromBigEndian(Temp);
				return true;
			}
			return false;
		}

		UE_API bool ReadVersionAndFlags(uint8& OutVersion, uint32& OutFlags);
		UE_API bool ReadString(FString& OutString, uint16 InNumBytes);
		UE_API bool ReadStringUTF8(FString& OutString, int32 InNumBytes);
		UE_API bool ReadStringUTF16(FString& OutString, int32 InNumBytes);
		UE_API bool ReadNullTerminatedStringUTF8(FString& OutString);
		UE_API bool ReadBytes(void* OutBuffer, int32 InNumBytes);
		UE_API bool ReadAsNumber(int64& OutValue, int32 InNumBytes);
		UE_API bool ReadAsNumber(uint64& OutValue, int32 InNumBytes);
		UE_API bool ReadAsNumber(float& OutValue);
		UE_API bool ReadAsNumber(double& OutValue);
		bool SkipBytes(int32 InNumBytes)
		{
			return ReadData(nullptr, InNumBytes) == InNumBytes;
		}
	private:
		template <typename T>
		T ValueFromBigEndian(const T value)
		{ return MP4Utilities::GetFromBigEndian(value); }
		UE_API int32 ReadData(void* IntoBuffer, int32 NumBytesToRead);

		const uint8* DataPtr = nullptr;
		int64 DataSize = 0;
		int64 CurrentOffset = 0;
	};


	class FMP4BoxLocator
	{
	public:
		FMP4BoxLocator() = default;
		~FMP4BoxLocator() = default;

		bool LocateAndReadRootBoxes(TArray<TSharedPtr<FMP4BoxData, ESPMode::ThreadSafe>>& OutBoxInfos, const TSharedPtr<IMP4DataReaderBase, ESPMode::ThreadSafe>& InDataReader, const TArray<uint32>& InFirstBoxes, const TArray<uint32>& InStopAfterBoxes, bool bStopWithMDAT, const TArray<uint32>& InReadDataOfBoxes, IMP4DataReaderBase::FCancellationCheckDelegate InCheckCancellationDelegate)
		{ return LocateAndReadRootBoxesInternal(OutBoxInfos, InDataReader, InFirstBoxes, InStopAfterBoxes, bStopWithMDAT, true, InReadDataOfBoxes, InCheckCancellationDelegate); }
		FString GetLastError() const
		{ return LastError; }

		static bool LocateAndReadRootBoxesFromBuffer(TArray<TSharedPtr<FMP4BoxData, ESPMode::ThreadSafe>>& OutBoxInfos, TConstArrayView<const uint8> InData)
		{ return FMP4BoxLocator().LocateAndReadRootBoxesInternal(OutBoxInfos, FMP4BufferDataReader::Create(InData), TArray<uint32>(), TArray<uint32>(), false, true, TArray<uint32>(), IMP4DataReaderBase::FCancellationCheckDelegate()); }
		static bool LocateAndReadBoxesFromBuffer(TArray<TSharedPtr<FMP4BoxData, ESPMode::ThreadSafe>>& OutBoxInfos, TConstArrayView<const uint8> InData)
		{ return FMP4BoxLocator().LocateAndReadRootBoxesInternal(OutBoxInfos, FMP4BufferDataReader::Create(InData), TArray<uint32>(), TArray<uint32>(), false, false, TArray<uint32>(), IMP4DataReaderBase::FCancellationCheckDelegate()); }
	private:
		UE_API bool LocateAndReadRootBoxesInternal(TArray<TSharedPtr<FMP4BoxData, ESPMode::ThreadSafe>>& OutBoxInfos, const TSharedPtr<IMP4DataReaderBase, ESPMode::ThreadSafe>& InDataReader, const TArray<uint32>& InFirstBoxes, const TArray<uint32>& InStopAfterBoxes, bool bStopWithMDAT, bool bCheckWellKnown, const TArray<uint32>& InReadDataOfBoxes, IMP4DataReaderBase::FCancellationCheckDelegate InCheckCancellationDelegate);
		FString LastError;
		int64 CurrentOffset = 0;
	};





	static int64 ConvertToTimescale(uint32 InTargetDenominator, int64 InSourceNumerator, uint32 InSourceDenominator)
	{
		if (InTargetDenominator == InSourceDenominator)
		{
			return InSourceNumerator;
		}
		else if (InSourceNumerator == 0)
		{
			return 0;
		}
		else if (InSourceDenominator == 0 || InTargetDenominator == 0)
		{
			return InSourceNumerator >= 0 ? 0x7fffffffffffffffLL : -0x7fffffffffffffffLL;
		}
		bool bIsNeg = InSourceNumerator < 0;
		TBigInt<128> n(bIsNeg ? -InSourceNumerator : InSourceNumerator);
		TBigInt<128> d(InSourceDenominator);
		TBigInt<128> s(InTargetDenominator);
		n *= s;
		n /= d;
		int64 r = n.ToInt();
		return bIsNeg ? -r : r;
	}



	class FFractionalTime
	{
	public:
		FFractionalTime() = default;
		FFractionalTime(int64 n, uint32 d) : Numerator(n), Denominator(d)
		{ }
		void SetFromND(int64 InNumerator, uint32 InDenominator)
		{
			Numerator = InNumerator;
			Denominator = InDenominator;
		}
		bool IsValid() const
		{ return Denominator != 0; }
		int64 GetNumerator() const
		{ return Numerator; }
		uint32 GetDenominator() const
		{ return Denominator; }
		void SetNumerator(int64 InNumerator)
		{ Numerator = InNumerator; }
		FTimespan GetAsTimespan() const
		{ return FTimespan(ConvertToTimescale(ETimespan::TicksPerSecond, Numerator, Denominator)); }
		int64 GetAsTimebase(uint32 CustomTimebase) const
		{ return ConvertToTimescale(CustomTimebase, Numerator, Denominator); }
	private:
		int64 Numerator = 0;
		uint32 Denominator = 0;
	};

} // namespace MP4Utilities

#undef UE_API
